{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Operator Stack",
	"Description": "Allows you to display linked objects in a container with an intuitive UI",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "OperatorStackEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "CustomDetailsView",
			"Enabled": true
		}
	]
}