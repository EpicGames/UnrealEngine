// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class OperatorStackEditor : ModuleRules
{
	public OperatorStackEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"CoreUObject",
				"CustomDetailsView",
				"EditorSubsystem",
				"Engine",
				"InputCore",
				"LevelEditor",
				"Projects",
				"PropertyEditor",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"ToolWidgets",
				"TypedElementRuntime",
				"UnrealEd",
				"WorkspaceMenuStructure"
			}
		);
	}
}
