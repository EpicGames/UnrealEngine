// Copyright Epic Games, Inc. All Rights Reserved.

#include "Builders/OperatorStackEditorBodyBuilder.h"
#include "Items/CustomDetailsViewItemId.h"

FOperatorStackEditorBodyBuilder& FOperatorStackEditorBodyBuilder::SetCustomWidget(TSharedPtr<SWidget> InWidget)
{
	CustomWidget = InWidget;
	return *this;
}

FOperatorStackEditorBodyBuilder& FOperatorStackEditorBodyBuilder::SetShowDetailsView(bool bInShowDetailsView)
{
	bShowDetailsView = bInShowDetailsView;
	return *this;
}

FOperatorStackEditorBodyBuilder& FOperatorStackEditorBodyBuilder::SetDetailsViewItem(TSharedPtr<FOperatorStackEditorItem> InItem)
{
	DetailsViewItem = InItem;
	return *this;
}

FOperatorStackEditorBodyBuilder& FOperatorStackEditorBodyBuilder::DisallowProperty(FProperty* InProperty)
{
	DisallowedDetailsViewItems.Add(MakeShared<FCustomDetailsViewItemId>(FCustomDetailsViewItemId::MakePropertyId(InProperty)));
	return *this;
}

FOperatorStackEditorBodyBuilder& FOperatorStackEditorBodyBuilder::DisallowCategory(const FName& InCategory)
{
	DisallowedDetailsViewItems.Add(MakeShared<FCustomDetailsViewItemId>(FCustomDetailsViewItemId::MakeCategoryId(InCategory)));
	return *this;
}

FOperatorStackEditorBodyBuilder& FOperatorStackEditorBodyBuilder::AllowProperty(FProperty* InProperty)
{
	AllowedDetailsViewItems.Add(MakeShared<FCustomDetailsViewItemId>(FCustomDetailsViewItemId::MakePropertyId(InProperty)));
	return *this;
}

FOperatorStackEditorBodyBuilder& FOperatorStackEditorBodyBuilder::AllowCategory(const FName& InCategory)
{
	AllowedDetailsViewItems.Add(MakeShared<FCustomDetailsViewItemId>(FCustomDetailsViewItemId::MakeCategoryId(InCategory)));
	return *this;
}

FOperatorStackEditorBodyBuilder& FOperatorStackEditorBodyBuilder::ExpandProperty(FProperty* InProperty)
{
	ExpandedDetailsViewItems.Add(MakeShared<FCustomDetailsViewItemId>(FCustomDetailsViewItemId::MakePropertyId(InProperty)));
	return *this;
}

FOperatorStackEditorBodyBuilder& FOperatorStackEditorBodyBuilder::CollapseProperty(FProperty* InProperty)
{
	CollapsedDetailsViewItems.Add(MakeShared<FCustomDetailsViewItemId>(FCustomDetailsViewItemId::MakePropertyId(InProperty)));
	return *this;
}

FOperatorStackEditorBodyBuilder& FOperatorStackEditorBodyBuilder::SetEmptyBodyText(const FText& InText)
{
	EmptyBodyText = InText;
	return *this;
}
