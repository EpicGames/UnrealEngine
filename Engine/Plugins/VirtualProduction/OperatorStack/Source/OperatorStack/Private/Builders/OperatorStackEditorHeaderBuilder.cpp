// Copyright Epic Games, Inc. All Rights Reserved.

#include "Builders/OperatorStackEditorHeaderBuilder.h"

#include "Items/CustomDetailsViewItemId.h"

FOperatorStackEditorHeaderBuilder& FOperatorStackEditorHeaderBuilder::SetIcon(const FSlateBrush* InIcon)
{
	Icon = InIcon;
	return *this;
}

FOperatorStackEditorHeaderBuilder& FOperatorStackEditorHeaderBuilder::SetLabel(const FText& InLabel)
{
	Label = InLabel;
	return *this;
}

FOperatorStackEditorHeaderBuilder& FOperatorStackEditorHeaderBuilder::SetTooltip(const FText& InTooltip)
{
	Tooltip = InTooltip;
	return *this;
}

FOperatorStackEditorHeaderBuilder& FOperatorStackEditorHeaderBuilder::SetBorderColor(const FLinearColor& InColor)
{
	BorderColor = InColor;
	return *this;
}

FOperatorStackEditorHeaderBuilder& FOperatorStackEditorHeaderBuilder::SetStartsExpanded(bool bInExpanded)
{
	bStartsExpanded = bInExpanded;
	return *this;
}

FOperatorStackEditorHeaderBuilder& FOperatorStackEditorHeaderBuilder::SetDraggable(bool bInDraggable)
{
	bDraggable = bInDraggable;
	return *this;
}

FOperatorStackEditorHeaderBuilder& FOperatorStackEditorHeaderBuilder::SetCommandList(TSharedPtr<FUICommandList> InCommands)
{
	CommandList = InCommands;
	return *this;
}

FOperatorStackEditorHeaderBuilder& FOperatorStackEditorHeaderBuilder::SetContextMenu(const FName& InToolMenuName)
{
	ContextMenuName = InToolMenuName;
	return *this;
}

FOperatorStackEditorHeaderBuilder& FOperatorStackEditorHeaderBuilder::SetCustomWidget(TSharedPtr<SWidget> InWidget)
{
	CustomWidget = InWidget;
	return *this;
}

FOperatorStackEditorHeaderBuilder& FOperatorStackEditorHeaderBuilder::SetMessageBox(const TAttribute<EOperatorStackEditorMessageType>& InType, const TAttribute<FText>& InText)
{
	MessageBoxType = InType;
	MessageBoxText = InText;
	return *this;
}

FOperatorStackEditorHeaderBuilder& FOperatorStackEditorHeaderBuilder::SetProperty(FProperty* InProperty)
{
	Property = MakeShared<FCustomDetailsViewItemId>(FCustomDetailsViewItemId::MakePropertyId(InProperty));
	return *this;
}

FOperatorStackEditorHeaderBuilder& FOperatorStackEditorHeaderBuilder::SetToolbarMenu(const FName& InActionMenuName)
{
	ToolbarMenuName = InActionMenuName;
	return *this;
}

FOperatorStackEditorHeaderBuilder& FOperatorStackEditorHeaderBuilder::SetToolMenu(const FName& InToolMenuName, const FText& InToolMenuLabel, const FSlateBrush* InToolMenuIcon)
{
	ToolMenuName = InToolMenuName;
	ToolMenuLabel = InToolMenuLabel;
	ToolMenuIcon = InToolMenuIcon;
	return *this;
}

FOperatorStackEditorHeaderBuilder& FOperatorStackEditorHeaderBuilder::SetSearchAllowed(bool bInSearchable)
{
	bSearchAllowed = bInSearchable;
	return *this;
}

FOperatorStackEditorHeaderBuilder& FOperatorStackEditorHeaderBuilder::SetSearchKeywords(const TSet<FString>& InKeywords)
{
	SearchKeywords = InKeywords;
	return *this;
}

FOperatorStackEditorHeaderBuilder& FOperatorStackEditorHeaderBuilder::SetSearchPinnedKeywords(const TSet<FString>& InKeywords)
{
	SearchPinnedKeywords = InKeywords;
	return *this;
}

FOperatorStackEditorHeaderBuilder& FOperatorStackEditorHeaderBuilder::SetExpandable(bool bInExpandable)
{
	bExpandable = bInExpandable;
	return *this;
}
