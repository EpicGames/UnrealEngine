// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/OperatorStackEditorStackCustomization.h"
#include "Styling/SlateIconFinder.h"
#include "Subsystems/OperatorStackEditorSubsystem.h"

bool UOperatorStackEditorStackCustomization::RegisterCustomizationFor(const UStruct* InItemDefinition)
{
	bool bAlreadyInSet = false;
	SupportedDefinitions.Add(InItemDefinition, &bAlreadyInSet);
	return !bAlreadyInSet;
}

bool UOperatorStackEditorStackCustomization::RegisterCustomizationFor(const FFieldClass* InItemDefinition)
{
	bool bAlreadyInSet = false;
	SupportedFieldClasses.Add(InItemDefinition, &bAlreadyInSet);
	return !bAlreadyInSet;
}

bool UOperatorStackEditorStackCustomization::UnregisterCustomizationFor(const UStruct* InItemDefinition)
{
	return SupportedDefinitions.Remove(InItemDefinition) > 0;
}

bool UOperatorStackEditorStackCustomization::UnregisterCustomizationFor(const FFieldClass* InItemDefinition)
{
	return SupportedFieldClasses.Remove(InItemDefinition) > 0;
}

bool UOperatorStackEditorStackCustomization::IsCustomizationSupportedFor(const FOperatorStackEditorItemPtr& InItem) const
{
	if (!InItem.IsValid())
	{
		return false;
	}

	if (const UStruct* ItemStruct = InItem->GetValueType().Get<UStruct>())
	{
		if (SupportedDefinitions.Contains(ItemStruct))
		{
			return true;
		}

		for (const TObjectPtr<const UStruct>& SupportedDefinition : SupportedDefinitions)
		{
			if (ItemStruct->IsChildOf(SupportedDefinition.Get()))
			{
				return true;
			}
		}
	}

	if (const FFieldClass* ItemClass = InItem->GetValueType().Get<FFieldClass>())
	{
		if (SupportedFieldClasses.Contains(ItemClass))
		{
			return true;
		}

		for (const FFieldClass* SupportedFieldClass : SupportedFieldClasses)
		{
			if (ItemClass->IsChildOf(SupportedFieldClass))
			{
				return true;
			}
		}
	}

	return false;
}

const FSlateBrush* UOperatorStackEditorStackCustomization::GetIcon() const
{
	return FSlateIconFinder::FindIconBrushForClass(GetClass());
}

void UOperatorStackEditorStackCustomization::RefreshActiveSelection(UObject* InContext, bool bInForce) const
{
	if (!IsValid(InContext))
	{
		return;
	}

	UOperatorStackEditorSubsystem* Subsystem = UOperatorStackEditorSubsystem::Get();

	if (!Subsystem)
	{
		return;
	}

	Subsystem->RefreshCustomizationWidget(InContext, bInForce);
}

void UOperatorStackEditorStackCustomization::FocusCustomization(const UObject* InContext) const
{
	if (!IsValid(InContext))
	{
		return;
	}

	UOperatorStackEditorSubsystem* Subsystem = UOperatorStackEditorSubsystem::Get();
	if (!Subsystem)
	{
		return;
	}

	Subsystem->FocusCustomizationWidget(InContext, Identifier);
}
