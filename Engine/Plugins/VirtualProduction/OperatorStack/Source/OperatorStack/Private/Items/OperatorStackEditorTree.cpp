// Copyright Epic Games, Inc. All Rights Reserved.

#include "Items/OperatorStackEditorTree.h"

#include "Customizations/OperatorStackEditorStackCustomization.h"

FOperatorStackEditorTree::FOperatorStackEditorTreeNode::FOperatorStackEditorTreeNode(int32 InItemIndex, int32 InParentIndex)
{
	check(InItemIndex >= 0)

	ItemIndex = InItemIndex;
	ParentIndex = InParentIndex;
}

FOperatorStackEditorTree::FOperatorStackEditorTree(UOperatorStackEditorStackCustomization* InCustomization, FOperatorStackEditorContextPtr InContext)
	: CustomizationWeak(InCustomization)
{
	check(InCustomization && InContext.IsValid());

	ContextWeak = InContext;

	// Get root items
	TSharedPtr<FOperatorStackEditorItem> RootItem;
	if (InCustomization->GetRootItem(*InContext.Get(), RootItem)
		&& RootItem.IsValid()
		&& RootItem->HasValue())
	{
		check(InCustomization->IsCustomizationSupportedFor(RootItem))

		Items.Empty(1);

		BuildTreeInternal({RootItem}, INDEX_NONE);
	}
}

FOperatorStackEditorItemPtr FOperatorStackEditorTree::GetRootItem() const
{
	if (RootNode.IsValid() && Items.IsValidIndex(RootNode->ItemIndex))
	{
		return Items[RootNode->ItemIndex];
	}

	return nullptr;
}

TArray<FOperatorStackEditorItemPtr> FOperatorStackEditorTree::GetChildrenItems(FOperatorStackEditorItemPtr InItem) const
{
	TArray<FOperatorStackEditorItemPtr> Children;

	const int32 Index = Items.Find(InItem);

	if (Nodes.IsValidIndex(Index))
	{
		Children.Reserve(Nodes[Index]->ChildrenIndices.Num());

		for (const int32 ChildIndex : Nodes[Index]->ChildrenIndices)
		{
			check(Items.IsValidIndex(ChildIndex))

			Children.Add(Items[ChildIndex]);
		}
	}

	return Children;
}

FOperatorStackEditorItemPtr FOperatorStackEditorTree::GetParentItem(FOperatorStackEditorItemPtr InItem) const
{
	const int32 Index = Items.Find(InItem);

	if (Nodes.IsValidIndex(Index))
	{
		const int32 ParentIndex = Nodes[Index]->ParentIndex;

		if (Items.IsValidIndex(ParentIndex))
		{
			return Items[Nodes[Index]->ParentIndex];
		}
	}

	return nullptr;
}

TArray<FOperatorStackEditorItemPtr> FOperatorStackEditorTree::GetLeafItems() const
{
	TArray<FOperatorStackEditorItemPtr> LeafItems;

	for (const FOperatorStackEditorTreeNodePtr& Node : Nodes)
	{
		if (Node->ChildrenIndices.IsEmpty())
		{
			check(Items.IsValidIndex(Node->ItemIndex))

			LeafItems.Add(Items[Node->ItemIndex]);
		}
	}

	return LeafItems;
}

TConstArrayView<FOperatorStackEditorItemPtr> FOperatorStackEditorTree::GetAllItems() const
{
	return Items;
}

const FOperatorStackEditorContext& FOperatorStackEditorTree::GetContext() const
{
	const FOperatorStackEditorContextPtr Context = ContextWeak.Pin();
	check(Context.IsValid())
	return *Context;
}

UOperatorStackEditorStackCustomization* FOperatorStackEditorTree::GetCustomization() const
{
	return CustomizationWeak.Get();
}

bool FOperatorStackEditorTree::Contains(FOperatorStackEditorItemPtr InItem) const
{
	if (!InItem.IsValid())
	{
		return false;
	}

	return Items.ContainsByPredicate([InItem](FOperatorStackEditorItemPtr InTreeItem)
	{
		if (InTreeItem.IsValid())
		{
			return *InTreeItem.Get() == *InItem.Get();
		}

		return false;
	});
}

TArray<FOperatorStackEditorItemPtr> FOperatorStackEditorTree::GetSupportedChildrenItems(const FOperatorStackEditorItemPtr& InParentItem) const
{
	TArray<FOperatorStackEditorItemPtr> ChildrenPtr;

	if (!InParentItem.IsValid() || !InParentItem->HasValue())
	{
		return ChildrenPtr;
	}

	const UOperatorStackEditorStackCustomization* StackCustomization = CustomizationWeak.Get();
	StackCustomization->GetChildrenItem(InParentItem, ChildrenPtr);

	ChildrenPtr.RemoveAll([StackCustomization](const FOperatorStackEditorItemPtr& InItem)
	{
		return !InItem.IsValid() || !InItem->HasValue() || !StackCustomization->IsCustomizationSupportedFor(InItem);
	});

	return ChildrenPtr;
}

void FOperatorStackEditorTree::BuildTreeInternal(const TArray<FOperatorStackEditorItemPtr>& InSupportedItems, int32 InParentIndex)
{
	for (const FOperatorStackEditorItemPtr& SupportedItem : InSupportedItems)
	{
		// Do not add item again
		if (!SupportedItem.IsValid()
			|| !SupportedItem->HasValue()
			|| Contains(SupportedItem))
		{
			continue;
		}

		const int32 ItemIndex = Items.Add(SupportedItem);

		FOperatorStackEditorTreeNodePtr Node = MakeShared<FOperatorStackEditorTreeNode>(ItemIndex, InParentIndex);
		Nodes.Add(Node);

		if (InParentIndex == INDEX_NONE)
		{
			check(!RootNode.IsValid())
			RootNode = Node;
		}
		else
		{
			check(Nodes.IsValidIndex(InParentIndex))

			Nodes[InParentIndex]->ChildrenIndices.AddUnique(ItemIndex);
		}

		BuildTreeInternal(GetSupportedChildrenItems(SupportedItem), ItemIndex);
	}
}
