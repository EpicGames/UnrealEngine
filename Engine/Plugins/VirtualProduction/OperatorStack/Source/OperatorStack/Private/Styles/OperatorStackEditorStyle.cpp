// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styles/OperatorStackEditorStyle.h"
#include "Misc/Paths.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateStyleRegistry.h"

FOperatorStackEditorStyle::FOperatorStackEditorStyle()
	: FSlateStyleSet(TEXT("OperatorStackEditor"))
{
	const FLinearColor BackgroundColor = COLOR("#1A1A1AFF");
	
	Set("HoverColor", COLOR("#808080FF"));
	Set("PropertiesBackgroundColor", COLOR("#1A1A1AFF"));
	Set("BackgroundColor", BackgroundColor);
	Set("ForegroundColor", COLOR("#242424FF"));
	
	const FTableViewStyle DefaultTreeViewStyle = FTableViewStyle()
		.SetBackgroundBrush(FSlateColorBrush(BackgroundColor));
	
	Set("ListViewStyle", DefaultTreeViewStyle);
	
	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FOperatorStackEditorStyle::~FOperatorStackEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}