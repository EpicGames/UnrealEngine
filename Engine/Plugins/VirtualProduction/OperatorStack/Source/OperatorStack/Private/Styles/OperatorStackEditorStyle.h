// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FOperatorStackEditorStyle : public FSlateStyleSet
{
public:
	FOperatorStackEditorStyle();
	
	virtual ~FOperatorStackEditorStyle() override;
	
	static FOperatorStackEditorStyle& Get()
	{
		static FOperatorStackEditorStyle StyleSet;
		return StyleSet;
	}
};
