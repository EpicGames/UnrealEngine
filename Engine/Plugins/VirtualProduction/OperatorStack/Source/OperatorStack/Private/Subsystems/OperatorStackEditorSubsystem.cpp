// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/OperatorStackEditorSubsystem.h"

#include "Customizations/OperatorStackEditorStackCustomization.h"
#include "Editor.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/SOperatorStackEditorPanel.h"
#include "Widgets/SOperatorStackEditorWidget.h"
#include "Widgets/Tabs/OperatorStackEditorTabInstance.h"

DEFINE_LOG_CATEGORY_STATIC(LogOperatorStackEditorSubsystem, Log, All);

const FName UOperatorStackEditorSubsystem::TabId = TEXT("OperatorStackEditorTab");
UOperatorStackEditorSubsystem::FOnOperatorStackSpawned UOperatorStackEditorSubsystem::OnOperatorStackSpawnedDelegate;

UOperatorStackEditorSubsystem* UOperatorStackEditorSubsystem::Get()
{
	if (GEditor)
	{
		return GEditor->GetEditorSubsystem<UOperatorStackEditorSubsystem>();
	}
	return nullptr;
}

void UOperatorStackEditorSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	ScanForStackCustomizations();

	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditorModule.OnLevelEditorCreated().AddUObject(this, &UOperatorStackEditorSubsystem::OnLevelEditorCreated);
}

void UOperatorStackEditorSubsystem::Deinitialize()
{
	Super::Deinitialize();

	CustomizationStacks.Empty();
	CustomizationWidgets.Empty();
	TabInstances.Empty();
	
	if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor"))
	{
		LevelEditorModule->OnLevelEditorCreated().RemoveAll(this);
	}
}

bool UOperatorStackEditorSubsystem::RegisterStackCustomization(const TSubclassOf<UOperatorStackEditorStackCustomization> InStackCustomizationClass)
{
	if (!InStackCustomizationClass->IsChildOf(UOperatorStackEditorStackCustomization::StaticClass())
		|| InStackCustomizationClass->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
	{
		return false;
	}

	UOperatorStackEditorStackCustomization* StackCustomization = InStackCustomizationClass->GetDefaultObject<UOperatorStackEditorStackCustomization>();
	if (!StackCustomization)
	{
		return false;
	}

	const FName StackIdentifier = StackCustomization->GetIdentifier();

	if (CustomizationStacks.Contains(StackIdentifier))
	{
		return false;
	}

	CustomizationStacks.Add(StackIdentifier, StackCustomization);

	UE_LOGF(LogOperatorStackEditorSubsystem, Display, "OperatorStack customization registered : Class %ls - Identifier %ls", *InStackCustomizationClass->GetName(), *StackIdentifier.ToString());

	return true;
}

bool UOperatorStackEditorSubsystem::UnregisterStackCustomization(const TSubclassOf<UOperatorStackEditorStackCustomization> InStackCustomizationClass)
{
	const UOperatorStackEditorStackCustomization* StackCustomization = InStackCustomizationClass->GetDefaultObject<UOperatorStackEditorStackCustomization>();
	if (!StackCustomization)
	{
		return false;
	}

	const FName StackIdentifier = StackCustomization->GetIdentifier();

    if (!CustomizationStacks.Contains(StackIdentifier))
    {
    	return false;
    }

    CustomizationStacks.Remove(StackIdentifier);

	UE_LOGF(LogOperatorStackEditorSubsystem, Display, "OperatorStack customization unregistered : Class %ls - Identifier %ls", *InStackCustomizationClass->GetName(), *StackIdentifier.ToString());

    return true;
}

TSharedRef<SOperatorStackEditorWidget> UOperatorStackEditorSubsystem::GenerateWidget()
{
	// Find new id available
	int32 NewId = 0;
	while (CustomizationWidgets.Contains(NewId))
	{
		NewId++;
	}

	TSharedRef<SOperatorStackEditorWidget> NewWidget = SNew(SOperatorStackEditorPanel)
		.PanelId(NewId);

	CustomizationWidgets.Add(NewId, NewWidget);

	return NewWidget;
}

TSharedPtr<SOperatorStackEditorWidget> UOperatorStackEditorSubsystem::FindWidget(int32 InId)
{
	if (const TWeakPtr<SOperatorStackEditorWidget>* WidgetWeak = CustomizationWidgets.Find(InId))
	{
		return WidgetWeak->Pin();
	}

	return nullptr;
}

TArray<TSharedPtr<SOperatorStackEditorWidget>> UOperatorStackEditorSubsystem::FindWidgets(const UWorld* InContext)
{
	TArray<TSharedPtr<SOperatorStackEditorWidget>> Widgets;
	Widgets.Reserve(TabInstances.Num());

	if (IsValid(InContext))
	{
		for (TSharedPtr<FOperatorStackEditorTabInstance>& TabInstance : TabInstances)
		{
			if (!TabInstance.IsValid())
			{
				continue;
			}

			TSharedPtr<ILevelEditor> LevelEditor = TabInstance->GetLevelEditor();
			if (!LevelEditor.IsValid())
			{
				continue;
			}

			if (LevelEditor->GetWorld() == InContext)
			{
				Widgets.Emplace(TabInstance->GetOperatorStackEditorWidget());
			}
		}
	}

	return Widgets;
}

bool UOperatorStackEditorSubsystem::ForEachCustomization(TFunctionRef<bool(UOperatorStackEditorStackCustomization*)> InFunction) const
{
	TArray<UOperatorStackEditorStackCustomization*> Customizations;

	for (const TPair<FName, TObjectPtr<UOperatorStackEditorStackCustomization>>& CustomizationPair : CustomizationStacks)
	{
		if (IsValid(CustomizationPair.Value))
		{
			Customizations.Add(CustomizationPair.Value);
		}
	}

	Customizations.Sort([](const UOperatorStackEditorStackCustomization& InA, const UOperatorStackEditorStackCustomization& InB)
	{
		return InA.GetPriority() > InB.GetPriority();
	});

	for (UOperatorStackEditorStackCustomization* Customization : Customizations)
	{
		if (!InFunction(Customization))
		{
			return false;
		}
	}

	return true;
}

bool UOperatorStackEditorSubsystem::ForEachCustomizationWidget(TFunctionRef<bool(TSharedRef<SOperatorStackEditorWidget>)> InFunction) const
{
	TArray<TWeakPtr<SOperatorStackEditorWidget>> WeakWidgets;
	CustomizationWidgets.GenerateValueArray(WeakWidgets);

	for (TWeakPtr<SOperatorStackEditorWidget>& WeakWidget : WeakWidgets)
	{
		TSharedPtr<SOperatorStackEditorWidget> Widget = WeakWidget.Pin();

		if (Widget.IsValid())
		{
			if (!InFunction(Widget.ToSharedRef()))
			{
				return false;
			}
		}
	}

	return true;
}

void UOperatorStackEditorSubsystem::RefreshCustomizationWidget(UObject* InContext, bool bInForce)
{
	if (IsValid(InContext))
	{
		for (TSharedPtr<FOperatorStackEditorTabInstance>& TabInstance : TabInstances)
		{
			if (TabInstance.IsValid() && TabInstance->RefreshTab(InContext, bInForce))
			{
				return;
			}
		}
	}
}

void UOperatorStackEditorSubsystem::FocusCustomizationWidget(const UObject* InContext, FName InIdentifier)
{
	if (IsValid(InContext))
	{
		for (TSharedPtr<FOperatorStackEditorTabInstance>& TabInstance : TabInstances)
		{
			if (TabInstance.IsValid() && TabInstance->FocusTab(InContext, InIdentifier))
			{
				return;
			}
		}
	}
}

UOperatorStackEditorStackCustomization* UOperatorStackEditorSubsystem::GetCustomization(const FName& InName) const
{
	if (const TObjectPtr<UOperatorStackEditorStackCustomization>* Customization = CustomizationStacks.Find(InName))
	{
		return *Customization;
	}

	return nullptr;
}

void UOperatorStackEditorSubsystem::ScanForStackCustomizations()
{
	for (UClass* const Class : TObjectRange<UClass>())
	{
		if (Class && Class->IsChildOf(UOperatorStackEditorStackCustomization::StaticClass()))
		{
			const TSubclassOf<UOperatorStackEditorStackCustomization> ScannedClass(Class);
			RegisterStackCustomization(ScannedClass);
		}
	}
}

void UOperatorStackEditorSubsystem::OnWidgetDestroyed(int32 InPanelId)
{
	CustomizationWidgets.Remove(InPanelId);
}

void UOperatorStackEditorSubsystem::OnLevelEditorCreated(TSharedPtr<ILevelEditor> InLevelEditor)
{
	for (TArray<TSharedPtr<FOperatorStackEditorTabInstance>>::TIterator It(TabInstances); It; ++It)
	{
		if (!It->IsValid() || !(*It)->GetLevelEditor().IsValid())
		{
			It.RemoveCurrent();
		}
	}

	if (InLevelEditor.IsValid())
	{
		const TSharedRef<FOperatorStackEditorTabInstance> TabInstance = MakeShared<FOperatorStackEditorTabInstance>(InLevelEditor.ToSharedRef());
		TabInstances.Emplace(TabInstance);
		TabInstance->RegisterTab();
	}
}
