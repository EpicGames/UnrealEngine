// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"

enum class EItemDropZone;
struct FOperatorStackEditorItem;

/** A custom drag operation for operator stack items */
class FOperatorStackEditorDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FOperatorStackEditorDragDropOp, FDecoratedDragDropOp)

	static TSharedRef<FOperatorStackEditorDragDropOp> New(const TArray<TSharedPtr<FOperatorStackEditorItem>>& InItems)
	{
		TSharedRef<FOperatorStackEditorDragDropOp> DragDropOp = MakeShared<FOperatorStackEditorDragDropOp>();
		DragDropOp->DraggedItems = InItems;
		DragDropOp->CurrentIconColorAndOpacity = FSlateColor::UseForeground();
		DragDropOp->MouseCursor = EMouseCursor::GrabHandClosed;

		DragDropOp->SetupDefaults();
		DragDropOp->Construct();
		
		return DragDropOp;
	}

	const TArray<TSharedPtr<FOperatorStackEditorItem>>& GetDraggedItems() const
	{
		return DraggedItems;
	}

	void SetDropZone(const TOptional<EItemDropZone>& InDropZone)
	{
		DropZone = InDropZone;	
	}

	TOptional<EItemDropZone> GetDropZone() const
	{
		return DropZone;	
	}
protected:
	/** Dragged items */
	TArray<TSharedPtr<FOperatorStackEditorItem>> DraggedItems;

	TOptional<EItemDropZone> DropZone;
};
