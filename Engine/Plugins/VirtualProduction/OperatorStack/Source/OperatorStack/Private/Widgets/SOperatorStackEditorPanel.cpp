// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SOperatorStackEditorPanel.h"

#include "Containers/Ticker.h"
#include "Customizations/OperatorStackEditorStackCustomization.h"
#include "DetailColumnSizeData.h"
#include "SOperatorStackEditorStack.h"
#include "Subsystems/OperatorStackEditorSubsystem.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Text/STextBlock.h"

SOperatorStackEditorPanel::~SOperatorStackEditorPanel()
{
	if (UOperatorStackEditorSubsystem* EditorSubsystem = UOperatorStackEditorSubsystem::Get())
	{
		EditorSubsystem->OnWidgetDestroyed(PanelId);
	}
}

void SOperatorStackEditorPanel::Construct(const FArguments& InArgs)
{
	PanelId = InArgs._PanelId;

	// Generate default empty context
	Context = MakeShared<FOperatorStackEditorContext>();

	// Setup default column sizes
	DetailColumnSize = MakeShared<FDetailColumnSizeData>();
	DetailColumnSize->SetRightColumnMinWidth(60.0f);
	DetailColumnSize->SetValueColumnWidth(0.6f);

	ChildSlot
	[
		SNew(SVerticalBox)
		// Toolbar to switch between different stacks
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Fill)
			.Padding(0.f, SOperatorStackEditorStack::Padding)
			[
				SAssignNew(WidgetToolbar, SScrollBox)
				.Orientation(EOrientation::Orient_Horizontal)
				.ScrollBarThickness(FVector2D(1.f))
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SSeparator)
			.Orientation(Orient_Horizontal)
			.Thickness(3.f)
		]
		// Widget for each stacks
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			SAssignNew(WidgetSwitcher, SWidgetSwitcher)
		]
	];

	const UOperatorStackEditorSubsystem* EditorSubsystem = UOperatorStackEditorSubsystem::Get();
	EditorSubsystem->ForEachCustomization([this](UOperatorStackEditorStackCustomization* InCustomization)
	{
		AddSlot(InCustomization);
		return true;
	});

	UpdateSlots();
}

void SOperatorStackEditorPanel::SetContext(const FOperatorStackEditorContext& InContext)
{
	Context = MakeShared<FOperatorStackEditorContext>(InContext);
	RefreshContext();
}

void SOperatorStackEditorPanel::SetActiveCustomization(const FName& InCustomization)
{
	if (WidgetSwitcher.IsValid() && NamedStackIndexes.Contains(InCustomization))
	{
		const int32* WidgetIdx = NamedStackIndexes.Find(InCustomization);
		WidgetSwitcher->SetActiveWidgetIndex(*WidgetIdx);
	}
}

void SOperatorStackEditorPanel::SetToolbarCustomizations(const TArray<FName>& InCustomizations)
{
	if (!WidgetToolbar.IsValid() || !WidgetSwitcher.IsValid())
	{
		return;
	}

	WidgetToolbar->ClearChildren();
	for (int32 SlotIdx = 0; SlotIdx < WidgetSwitcher->GetNumWidgets(); SlotIdx++)
	{
		if (TSharedPtr<SWidget> SlotWidget = WidgetSwitcher->GetWidget(SlotIdx))
		{
			WidgetSwitcher->RemoveSlot(SlotWidget.ToSharedRef());
		}
	}

	const UOperatorStackEditorSubsystem* EditorSubsystem = UOperatorStackEditorSubsystem::Get();

	for (const FName& CustomizationName : InCustomizations)
	{
		const UOperatorStackEditorStackCustomization* Customization = EditorSubsystem->GetCustomization(CustomizationName);

		if (!Customization)
		{
			continue;
		}

		AddSlot(Customization);
	}

	UpdateSlots();
}

void SOperatorStackEditorPanel::GetToolbarCustomizations(TArray<UOperatorStackEditorStackCustomization*>& OutCustomizations)
{
	OutCustomizations.Empty(CustomizationStacks.Num());
	Algo::Transform(CustomizationStacks, OutCustomizations, [](const TSharedRef<SOperatorStackEditorStack>& InStack)
	{
		return InStack->GetStackCustomization();
	});
}

void SOperatorStackEditorPanel::ShowToolbarCustomization(FName InCustomization)
{
	if (const int32* CustomizationIdx = NamedStackIndexes.Find(InCustomization))
	{
		VisibleCustomizations.Add(*CustomizationIdx);
	}
}

void SOperatorStackEditorPanel::HideToolbarCustomization(FName InCustomization)
{
	if (const int32* CustomizationIdx = NamedStackIndexes.Find(InCustomization))
	{
		VisibleCustomizations.Remove(*CustomizationIdx);
	}
}

void SOperatorStackEditorPanel::SetToolbarVisibility(bool bInVisible)
{
	if (!WidgetToolbar.IsValid())
	{
		return;
	}

	WidgetToolbar->SetVisibility(bInVisible ? EVisibility::Visible : EVisibility::Collapsed);
}

void SOperatorStackEditorPanel::SetKeyframeHandler(TSharedPtr<IDetailKeyframeHandler> InKeyframeHandler)
{
	KeyframeHandler = InKeyframeHandler;
	RefreshContext();
}

void SOperatorStackEditorPanel::SetDetailColumnSize(TSharedPtr<FDetailColumnSizeData> InDetailColumnSize)
{
	DetailColumnSize = InDetailColumnSize;
	RefreshContext();
}

void SOperatorStackEditorPanel::SetPanelTag(FName InTag)
{
	PanelTag = InTag;
}

void SOperatorStackEditorPanel::RefreshContext()
{
	UpdateSlots();
}

const FOperatorStackEditorTree& SOperatorStackEditorPanel::GetItemTree(UOperatorStackEditorStackCustomization* InCustomization)
{
	const FOperatorStackEditorTree* Tree = CustomizationTrees.Find(InCustomization);
	check(Tree)
	return *Tree;
}

void SOperatorStackEditorPanel::SaveItemExpansionState(uint32 InItem, bool bInExpanded)
{
	ItemExpansionState.Add(InItem, bInExpanded);
}

bool SOperatorStackEditorPanel::GetItemExpansionState(uint32 InItem, bool& bOutExpanded)
{
	if (const bool* bExpanded = ItemExpansionState.Find(InItem))
	{
		bOutExpanded = *bExpanded;
		return true;
	}

	return false;
}

EVisibility SOperatorStackEditorPanel::GetToolbarButtonVisibility(int32 InIdx) const
{
	return VisibleCustomizations.Contains(InIdx) ? EVisibility::Visible : EVisibility::Collapsed;
}

void SOperatorStackEditorPanel::AddSlot(const UOperatorStackEditorStackCustomization* InCustomizationStack)
{
	if (!WidgetToolbar.IsValid() || !WidgetSwitcher.IsValid())
	{
		return;
	}

	const FText& Label = InCustomizationStack->GetLabel();
	const FName& Identifier = InCustomizationStack->GetIdentifier();
	const FSlateBrush* Icon = InCustomizationStack->GetIcon();

	// Add widget for stack content
	WidgetSwitcher->AddSlot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(SOperatorStackEditorStack::Padding, 0.f)
		[
			SNew(SBox)
			.Padding(0.f)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
		];

	int32 WidgetIdx = WidgetSwitcher->GetNumWidgets() - 1;

	// Add button to switch to new stack
	WidgetToolbar->AddSlot()
		.Padding(SOperatorStackEditorStack::Padding)
		[
			SNew(SBox)
			.Padding(FMargin(0.0f))
			[
				SNew(SCheckBox)
				.Style(FAppStyle::Get(), "DetailsView.SectionButton")
				.OnCheckStateChanged(this, &SOperatorStackEditorPanel::OnToolbarButtonClicked, WidgetIdx)
				.IsChecked(this, &SOperatorStackEditorPanel::IsToolbarButtonActive, WidgetIdx)
				.Visibility(this, &SOperatorStackEditorPanel::GetToolbarButtonVisibility, WidgetIdx)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2.f)
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.DesiredSizeOverride(FVector2D(12.f, 12.f))
						.Image(Icon)
					]
					+ SHorizontalBox::Slot()
					.FillWidth(1.f)
					.Padding(2.f)
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(Label)
					]
				]
			]
		];

	NamedStackIndexes.Emplace(Identifier, WidgetIdx);
	VisibleCustomizations.Add(WidgetIdx);
}

void SOperatorStackEditorPanel::UpdateSlots()
{
	const UOperatorStackEditorSubsystem* EditorSubsystem = UOperatorStackEditorSubsystem::Get();

	CustomizationStacks.Empty(NamedStackIndexes.Num());

	for (const TPair<FName, int32>& NamedSlot : NamedStackIndexes)
	{
		if (const TSharedPtr<SBox> Box = StaticCastSharedPtr<SBox>(WidgetSwitcher->GetWidget(NamedSlot.Value)))
		{
			UOperatorStackEditorStackCustomization* Customization = EditorSubsystem->GetCustomization(NamedSlot.Key);

			check(Customization)

			FOperatorStackEditorTree& ItemTree = CustomizationTrees.Add(Customization, FOperatorStackEditorTree(Customization, Context));

			TSharedRef<SOperatorStackEditorStack> Stack = SNew(SOperatorStackEditorStack, SharedThis(this), Customization, nullptr);
			CustomizationStacks.Add(Stack);

			Box->SetContent(
				Stack
			);

			if (Context.IsValid() && Customization->ShouldFocusCustomization(*Context))
			{
				SetActiveCustomization(Customization->GetIdentifier());
			}
		}
	}
}

void SOperatorStackEditorPanel::OnToolbarButtonClicked(ECheckBoxState InState, int32 InWidgetIdx)
{
	if (!WidgetSwitcher.IsValid() || InWidgetIdx == INDEX_NONE)
	{
		return;
	}

	WidgetSwitcher->SetActiveWidgetIndex(InWidgetIdx);

	// Refresh search for new active item
	FilterItemsAsync(LastSearch);
}

ECheckBoxState SOperatorStackEditorPanel::IsToolbarButtonActive(int32 InWidgetIdx) const
{
	if (!WidgetSwitcher.IsValid() || InWidgetIdx == INDEX_NONE)
	{
		return ECheckBoxState::Undetermined;
	}

	return WidgetSwitcher->GetActiveWidgetIndex() == InWidgetIdx
		? ECheckBoxState::Checked
		: ECheckBoxState::Unchecked;
}

void SOperatorStackEditorPanel::FilterItemsAsync(const FText& InNewSearch, float InDelay)
{
	LastSearch = InNewSearch;

	// Cancel previous search
	if (LastSearchHandle.IsValid())
	{
		FTSTicker::RemoveTicker(LastSearchHandle);
		LastSearchHandle.Reset();
	}

	// Start a new search on active items
	LastSearchHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSPLambda(this, [this](float)->bool
		{
			const int32 Index = WidgetSwitcher->GetActiveWidgetIndex();

			if (CustomizationStacks.IsValidIndex(Index))
			{
				CustomizationStacks[Index]->FilterItems(LastSearch);
			}

			return false;
		}
	), FMath::Abs(InDelay));
}
