// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomDetailsViewArgs.h"
#include "Contexts/OperatorStackEditorContext.h"
#include "Items/OperatorStackEditorTree.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SOperatorStackEditorWidget.h"

class FReply;
class SOperatorStackEditorStack;
class SScrollBox;
class SWidgetSwitcher;
class UOperatorStackEditorStackCustomization;
struct FButtonStyle;

/** This is the main panel that contains the stacks and switcher for them */
class SOperatorStackEditorPanel : public SOperatorStackEditorWidget
{
public:
	SLATE_BEGIN_ARGS(SOperatorStackEditorPanel) {}
		SLATE_ARGUMENT(int32, PanelId)
	SLATE_END_ARGS()

	virtual ~SOperatorStackEditorPanel() override;

	void Construct(const FArguments& InArgs);

	//~ Begin SOperatorStackEditorWidget
	virtual void SetContext(const FOperatorStackEditorContext& InContext) override;
	virtual void SetActiveCustomization(const FName& InCustomization) override;
	virtual void SetToolbarCustomizations(const TArray<FName>& InCustomizations) override;
	virtual void GetToolbarCustomizations(TArray<UOperatorStackEditorStackCustomization*>& OutCustomizations) override;
	virtual void ShowToolbarCustomization(FName InCustomization) override;
	virtual void HideToolbarCustomization(FName InCustomization) override;
	virtual void SetToolbarVisibility(bool bInVisible) override;
	virtual void SetKeyframeHandler(TSharedPtr<IDetailKeyframeHandler> InKeyframeHandler) override;
	virtual void SetDetailColumnSize(TSharedPtr<FDetailColumnSizeData> InDetailColumnSize) override;
	virtual void SetPanelTag(FName InTag) override;
	virtual void RefreshContext() override;
	virtual FOperatorStackEditorContextPtr GetContext() const override
	{
		return Context;
	}
	virtual int32 GetPanelId() const override
	{
		return PanelId;
	}
	virtual FName GetPanelTag() const override
	{
		return PanelTag;
	}
	//~ End SOperatorStackEditorWidget

	const FOperatorStackEditorTree& GetItemTree(UOperatorStackEditorStackCustomization* InCustomization);

	void SaveItemExpansionState(uint32 InItem, bool bInExpanded);

	bool GetItemExpansionState(uint32 InItem, bool& bOutExpanded);

	TSharedPtr<IDetailKeyframeHandler> GetKeyframeHandler() const
	{
		return KeyframeHandler;
	}

	TSharedPtr<FDetailColumnSizeData> GetDetailColumnSize() const
	{
		return DetailColumnSize;
	}

	const FText& GetLastSearch() const
	{
		return LastSearch;
	}

	void FilterItemsAsync(const FText& InNewSearch, float InDelay = 0.f);

protected:
	/** Is the toolbar button active */
	EVisibility GetToolbarButtonVisibility(int32 InIdx) const;

	/** Adds a new widget slot for toolbar and switcher */
	void AddSlot(const UOperatorStackEditorStackCustomization* InCustomizationStack);

	/** Update the slots content with actual widget */
	void UpdateSlots();

	/** Switch view when toolbar button is clicked */
	void OnToolbarButtonClicked(ECheckBoxState InState, int32 InWidgetIdx);

	/** Highlight active selection */
	ECheckBoxState IsToolbarButtonActive(int32 InWidgetIdx) const;

private:
	/** Contains the named stack views */
	TSharedPtr<SScrollBox> WidgetToolbar;

	/** Contains the different stack views */
	TSharedPtr<SWidgetSwitcher> WidgetSwitcher;

	/** Stack index with their identifier for updates */
	TMap<FName, int32> NamedStackIndexes;

	/** Customization visible to switch in toolbar */
	TSet<int32> VisibleCustomizations;

	/** Current context, could be null */
	FOperatorStackEditorContextPtr Context;

	/** Keyframe handler to be able to keyframe properties in items */
	TSharedPtr<IDetailKeyframeHandler> KeyframeHandler;

	/** Detail column size to have matching size across items */
	TSharedPtr<FDetailColumnSizeData> DetailColumnSize;

	/** The panel id to identify this widget */
	int32 PanelId = INDEX_NONE;

	/** The panel tag to identify this widget */
	FName PanelTag = NAME_None;

	/** Item trees containing items supported for each customization */
	TMap<TWeakObjectPtr<UOperatorStackEditorStackCustomization>, FOperatorStackEditorTree> CustomizationTrees;

	/** To check whether an item was expanded and restore it */
	TMap<uint32, bool> ItemExpansionState;

	/** Contains the customization stacks */
	TArray<TSharedRef<SOperatorStackEditorStack>> CustomizationStacks;

	/** Last search done within the stack */
	FText LastSearch = FText::GetEmpty();

	/** Last search request triggered */
	FTSTicker::FDelegateHandle LastSearchHandle;
};