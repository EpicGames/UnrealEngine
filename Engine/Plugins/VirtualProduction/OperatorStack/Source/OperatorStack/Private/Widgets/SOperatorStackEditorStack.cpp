// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SOperatorStackEditorStack.h"

#include "Builders/OperatorStackEditorBodyBuilder.h"
#include "Builders/OperatorStackEditorFooterBuilder.h"
#include "Builders/OperatorStackEditorHeaderBuilder.h"
#include "Contexts/OperatorStackEditorMenuContext.h"
#include "CustomDetailsViewModule.h"
#include "Customizations/OperatorStackEditorStackCustomization.h"
#include "Framework/Application/SlateApplication.h"
#include "ICustomDetailsView.h"
#include "Items/ICustomDetailsViewItem.h"
#include "Items/OperatorStackEditorTree.h"
#include "Items/OperatorStackEditorStructItem.h"
#include "SOperatorStackEditorPanel.h"
#include "SOperatorStackExpanderButton.h"
#include "SPositiveActionButton.h"
#include "Styles/OperatorStackEditorStyle.h"
#include "Styling/AppStyle.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/TableRows/SOperatorStackEditorStackRow.h"

#define LOCTEXT_NAMESPACE "OperatorStackEditorStack"

const TMap<EOperatorStackEditorMessageType, FLinearColor> SOperatorStackEditorStack::MessageBoxColors
{
	{EOperatorStackEditorMessageType::None, FLinearColor::Transparent},
	{EOperatorStackEditorMessageType::Info, FLinearColor::Blue.Desaturate(0.5)},
	{EOperatorStackEditorMessageType::Success, FLinearColor::Green.Desaturate(0.5)},
	{EOperatorStackEditorMessageType::Warning, FLinearColor::Yellow.Desaturate(0.5)},
	{EOperatorStackEditorMessageType::Error, FLinearColor::Red.Desaturate(0.5)},
};

const TMap<EOperatorStackEditorMessageType, const FSlateBrush*> SOperatorStackEditorStack::MessageBoxIcons
{
	{EOperatorStackEditorMessageType::None, nullptr},
	{EOperatorStackEditorMessageType::Info, FAppStyle::GetBrush("Icons.InfoWithColor")},
	{EOperatorStackEditorMessageType::Success, FAppStyle::GetBrush("Icons.SuccessWithColor")},
	{EOperatorStackEditorMessageType::Warning, FAppStyle::GetBrush("Icons.WarningWithColor")},
	{EOperatorStackEditorMessageType::Error, FAppStyle::GetBrush("Icons.ErrorWithColor")},
};

void SOperatorStackEditorStack::Construct(const FArguments& InArgs
    , const TSharedPtr<SOperatorStackEditorPanel> InMainPanel
    , TObjectPtr<UOperatorStackEditorStackCustomization> InCustomization
    , const FOperatorStackEditorItemPtr& InCustomizeItem)
{
	MainPanelWeak = InMainPanel;
	CustomizeItem = InCustomizeItem;
	StackCustomizationWeak = InCustomization;

	check(InCustomization && InMainPanel);

	const FOperatorStackEditorTree& ItemTree = InMainPanel->GetItemTree(InCustomization);

	if (InCustomizeItem.IsValid())
	{
		Items = ItemTree.GetChildrenItems(InCustomizeItem);
	}
	else if (const FOperatorStackEditorItemPtr RootItem = ItemTree.GetRootItem())
	{
		Items = {RootItem};
	}

	ChildSlot
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Fill)
	[
		GenerateStackWidget()
	];

	// Refresh search when we hit a leaf
	if (CustomizeItem.IsValid() && Items.IsEmpty())
	{
		InMainPanel->FilterItemsAsync(InMainPanel->GetLastSearch());
	}
}

FOperatorStackEditorContextPtr SOperatorStackEditorStack::GetContext() const
{
	if (const TSharedPtr<SOperatorStackEditorPanel> MainPanel = GetMainPanel())
	{
		return MainPanel->GetContext();
	}

	return nullptr;
}

TSharedRef<ITableRow> SOperatorStackEditorStack::OnGenerateRow(FOperatorStackEditorItemPtr InItem, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(SOperatorStackEditorStackRow, InOwnerTable, SharedThis(this), InItem);
}

TSharedPtr<SWidget> SOperatorStackEditorStack::GenerateHeaderWidget()
{
	UOperatorStackEditorStackCustomization* StackCustomization = GetStackCustomization();
	const FOperatorStackEditorContextPtr Context = GetContext();
	const TSharedPtr<SOperatorStackEditorPanel> MainPanel = GetMainPanel();

	if (!Context.IsValid())
	{
		return nullptr;
	}

	check(MainPanel.IsValid());

	FOperatorStackEditorHeaderBuilder HeaderBuilder;
	const FOperatorStackEditorTree& ItemTree = MainPanel->GetItemTree(StackCustomization);

	// We are in the top most stack, customize stack header
	if (!CustomizeItem.IsValid())
	{
		StackCustomization->CustomizeStackHeader(ItemTree, HeaderBuilder);
	}
	// We are in a child item, customize child header
	else
	{
		StackCustomization->CustomizeItemHeader(CustomizeItem, ItemTree, HeaderBuilder);
	}

	// Get Commands for this item
	CommandList = HeaderBuilder.GetCommandList();

	// Get Context menu name for this item
	ContextMenuName = HeaderBuilder.GetContextMenuName();

	// Get border color for item
	BorderColor = HeaderBuilder.GetBorderColor();

	const TSharedRef<SHorizontalBox> HorizontalHeaderWidget = SNew(SHorizontalBox);

	// Expansion button to show body and footer
	if (HeaderBuilder.GetExpandable())
	{
		// Find previous expansion state or use default
		bHeaderExpanded = HeaderBuilder.GetStartsExpanded();
		MainPanel->GetItemExpansionState(CustomizeItem->GetHash(), bHeaderExpanded);

		HorizontalHeaderWidget->AddSlot()
		.AutoWidth()
		.Padding(Padding)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(SOperatorStackExpanderButton)
			.StartsExpanded(bHeaderExpanded)
			.OnExpansionStateChanged(this, &SOperatorStackEditorStack::OnHeaderExpansionChanged)
		];
	}

	// Icon
	if (const FSlateBrush* Icon = HeaderBuilder.GetIcon())
	{
		HorizontalHeaderWidget->AddSlot()
		.AutoWidth()
		.Padding(Padding)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(Icon)
			.ToolTipText(HeaderBuilder.GetTooltip())
		];
	}

	// Label
	if (!HeaderBuilder.GetLabel().IsEmpty())
	{
		HorizontalHeaderWidget->AddSlot()
		.AutoWidth()
		.Padding(Padding)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(HeaderBuilder.GetLabel())
			.ToolTipText(HeaderBuilder.GetTooltip())
			.Justification(ETextJustify::Center)
		];
	}

	// Space here
	HorizontalHeaderWidget->AddSlot()
		.FillWidth(1.f)
		.Padding(Padding)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SNullWidget::NullWidget
		];

	// custom actions menu in header
	const FName& ActionMenuName = HeaderBuilder.GetToolbarMenuName();
	const UToolMenu* ActionMenu = !ActionMenuName.IsNone()
		? UToolMenus::Get()->FindMenu(ActionMenuName)
		: nullptr;

	if (ActionMenu)
	{
		HorizontalHeaderWidget->AddSlot()
		.Padding(Padding)
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.HeightOverride(25.f)
			[
				SNew(SScaleBox)
				.Stretch(EStretch::ScaleToFit)
				[
					GenerateMenuWidget(ActionMenuName)
				]
			]
		];
	}

	// Custom property in header
	const TSharedPtr<FCustomDetailsViewItemId> PropertyId = HeaderBuilder.GetProperty();
	if (CustomizeItem.IsValid() && PropertyId.IsValid())
	{
		const FCustomDetailsViewItemId& ItemId = *PropertyId.Get();

		FCustomDetailsViewArgs HeaderCustomViewArgs;

        HeaderCustomViewArgs.bShowCategories = false;
        HeaderCustomViewArgs.KeyframeHandler = MainPanel->GetKeyframeHandler();
        HeaderCustomViewArgs.bAllowGlobalExtensions = true;
        HeaderCustomViewArgs.ItemAllowList.Allow(ItemId);
        HeaderCustomViewArgs.WidgetTypeAllowList.Allow(ECustomDetailsViewWidgetType::Value);
        HeaderCustomViewArgs.WidgetTypeAllowList.Allow(ECustomDetailsViewWidgetType::Extensions);

		HeaderDetailsView = CreateDetailsView(HeaderCustomViewArgs, *CustomizeItem);

		if (const TSharedPtr<ICustomDetailsViewItem> HeaderCustomDetailsViewItem = HeaderDetailsView->FindItem(ItemId))
		{
			HeaderCustomDetailsViewItem->MakeWidget(nullptr, nullptr);

			// Add Property header widget (eg: checkbox for bool)
			const TSharedPtr<SWidget> HeaderPropertyValueWidget = HeaderCustomDetailsViewItem->GetWidget(ECustomDetailsViewWidgetType::Value);
			if (HeaderPropertyValueWidget.IsValid())
			{
				HorizontalHeaderWidget->AddSlot()
				.AutoWidth()
				.MaxWidth(25.0f)
				.Padding(Padding)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					HeaderPropertyValueWidget.ToSharedRef()
				];
			}

			// Add Expansion property widget (eg : keyframe icon)
			const TSharedPtr<SWidget> HeaderPropertyExtensionWidget = HeaderCustomDetailsViewItem->GetWidget(ECustomDetailsViewWidgetType::Extensions);
			if (HeaderPropertyExtensionWidget.IsValid())
			{
				HorizontalHeaderWidget->AddSlot()
				.AutoWidth()
				.MaxWidth(25.0f)
				.Padding(Padding)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					HeaderPropertyExtensionWidget.ToSharedRef()
				];
			}
		}
	}

	const TSharedPtr<SVerticalBox> VerticalHeaderWidget = SNew(SVerticalBox);

	if (HorizontalHeaderWidget->NumSlots() > 0)
	{
		VerticalHeaderWidget->AddSlot()
		.Padding(Padding)
		.AutoHeight()
		[
			HorizontalHeaderWidget
		];
	}

	// Custom search if allowed
	if (HeaderBuilder.GetSearchAllowed())
	{
		SearchableKeywords = HeaderBuilder.GetSearchKeywords();

		// Show search box only for root item, not per item
		if (!CustomizeItem.IsValid())
		{
			// Add search box
			VerticalHeaderWidget->AddSlot()
            .Padding(Padding)
            .AutoHeight()
            [
            	SAssignNew(SearchBox, SSearchBox)
	            .InitialText(MainPanel->GetLastSearch())
            	.HintText(LOCTEXT("OperatorStackEditorStackSearchHint", "Search items"))
            	.OnTextChanged(this, &SOperatorStackEditorStack::OnSearchTextChanged)
            	.OnTextCommitted(this, &SOperatorStackEditorStack::OnSearchTextCommitted)
            ];

			// Add scrollbox with pinned keywords
			if (!HeaderBuilder.GetSearchPinnedKeywords().IsEmpty())
			{
				const TSharedPtr<SScrollBox> SearchScroll = SNew(SScrollBox)
				.Orientation(EOrientation::Orient_Horizontal)
				.ScrollBarThickness(FVector2D(3.f));

				auto CreatePinnedButton = [this](const FString& InPinnedKeyword)->TSharedRef<SWidget>
				{
					return SNew(SBox)
						.Padding(FMargin(0.0f))
						[
							SNew(SCheckBox)
							.Style(FAppStyle::Get(), "DetailsView.SectionButton")
							.OnCheckStateChanged(this, &SOperatorStackEditorStack::OnSearchPinnedKeyword, InPinnedKeyword)
							[
								SNew(STextBlock)
								.TextStyle(FAppStyle::Get(), "SmallText")
								.Text(FText::FromString(InPinnedKeyword))
							]
						];
				};

				for (const FString& PinnedKeyword : HeaderBuilder.GetSearchPinnedKeywords())
				{
					SearchScroll->AddSlot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Fill)
					.Padding(Padding)
					[
						CreatePinnedButton(PinnedKeyword)
					];
				}

				VerticalHeaderWidget->AddSlot()
				.Padding(Padding)
				.AutoHeight()
				[
					SearchScroll.ToSharedRef()
				];
			}
		}
	}

	// Custom tool menu on next row
	const FName& ToolMenuName = HeaderBuilder.GetToolMenuName();
	const UToolMenu* ToolMenu = !ToolMenuName.IsNone()
		? UToolMenus::Get()->FindMenu(ToolMenuName)
		: nullptr;

	if (ToolMenu)
	{
		const TSharedRef<SWidget> MenuButtonWidget = SNew(SComboButton)
			.ToolTipText(HeaderBuilder.GetToolMenuLabel())
			.OnGetMenuContent(this, &SOperatorStackEditorStack::GenerateMenuWidget, ToolMenuName)
			.HasDownArrow(false)
			.ContentPadding(FMargin(Padding))
			.ButtonContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(Padding)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.DesiredSizeOverride(FVector2D(16.f, 16.f))
					.Image(HeaderBuilder.GetToolMenuIcon())
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				.Padding(Padding)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(HeaderBuilder.GetToolMenuLabel())
				]
			];

		VerticalHeaderWidget->AddSlot()
		.Padding(Padding)
		.AutoHeight()
		[
			SNew(SPositiveActionButton)
			.Icon(HeaderBuilder.GetToolMenuIcon())
			.Text(HeaderBuilder.GetToolMenuLabel())
			.ToolTipText(HeaderBuilder.GetToolMenuLabel())
			.OnGetMenuContent(this, &SOperatorStackEditorStack::GenerateMenuWidget, ToolMenuName)
		];
	}

	if (CustomizeItem.IsValid())
	{
		MessageBoxText = HeaderBuilder.GetMessageBoxText();
		MessageBoxType = HeaderBuilder.GetMessageBoxType();

		// Custom message box
		VerticalHeaderWidget->AddSlot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.AutoHeight()
		[
			SNew(SOverlay)
			.Visibility(this, &SOperatorStackEditorStack::GetMessageBoxVisibility)
			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SNew(SColorBlock)
				.Color(this, &SOperatorStackEditorStack::GetMessageBoxBackgroundColor)
			]
			+ SOverlay::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Fill)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(Padding)
				.AutoWidth()
				[
					SNew(SScaleBox)
					.Stretch(EStretch::ScaleToFit)
					.OverrideScreenSize(FVector2D(16.f))
					[
						SNew(SImage)
						.DesiredSizeOverride(FVector2D(16.f))
						.Visibility(this, &SOperatorStackEditorStack::GetMessageBoxIconVisibility)
						.Image(this, &SOperatorStackEditorStack::GetMessageBoxIcon)
					]
				]
				+ SHorizontalBox::Slot()
				.Padding(Padding)
				.FillWidth(1.f)
				[
					SNew(STextBlock)
					.Justification(ETextJustify::Left)
					.ColorAndOpacity(FLinearColor::White)
					.Text(HeaderBuilder.GetMessageBoxText())
				]
			]
		];
	}

	return SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("NoBorder"))
		.OnMouseButtonDown(this, &SOperatorStackEditorStack::OnHeaderMouseButtonDown)
		[
			VerticalHeaderWidget.ToSharedRef()
		];
}

TSharedPtr<SWidget> SOperatorStackEditorStack::GenerateBodyWidget()
{
	UOperatorStackEditorStackCustomization* StackCustomization = GetStackCustomization();
	const TSharedPtr<SOperatorStackEditorPanel> MainPanel = GetMainPanel();

	check(MainPanel.IsValid());

	const FOperatorStackEditorTree& ItemTree = MainPanel->GetItemTree(StackCustomization);

	FText EmptyBodyDefaultText = LOCTEXT("EmptyBodyText", "Select a supported item to display it here");

	auto GetItemWidget = [this, MainPanel](const FOperatorStackEditorBodyBuilder& InItem)->TSharedPtr<SWidget>
	{
		// We have set a custom widget for this body
		if (TSharedPtr<SWidget> CustomWidget = InItem.GetCustomWidget())
		{
			return CustomWidget;
		}

		// Only build details view if we allow it
		if (InItem.GetShowDetailsView())
		{
			FCustomDetailsViewArgs BodyCustomViewArgs;
			BodyCustomViewArgs.bShowCategories = false;
			BodyCustomViewArgs.KeyframeHandler = MainPanel->GetKeyframeHandler();
			BodyCustomViewArgs.bAllowGlobalExtensions = true;
			BodyCustomViewArgs.ColumnSizeData = MainPanel->GetDetailColumnSize();

			for (const TSharedRef<FCustomDetailsViewItemId>& DetailsViewId : InItem.GetDisallowedDetailsViewItems())
			{
				BodyCustomViewArgs.ItemAllowList.Disallow(DetailsViewId.Get());
			}

			for (const TSharedRef<FCustomDetailsViewItemId>& DetailsViewId : InItem.GetAllowedDetailsViewItems())
			{
				BodyCustomViewArgs.ItemAllowList.Allow(DetailsViewId.Get());
			}

			for (const TSharedRef<FCustomDetailsViewItemId>& DetailsViewId : InItem.GetCollapsedDetailsViewItems())
			{
				BodyCustomViewArgs.ExpansionState.Add(DetailsViewId.Get(), ECustomDetailsViewExpansion::Collapsed);
			}

			for (const TSharedRef<FCustomDetailsViewItemId>& DetailsViewId : InItem.GetExpandedDetailsViewItems())
			{
				BodyCustomViewArgs.ExpansionState.Add(DetailsViewId.Get(), ECustomDetailsViewExpansion::SelfAndChildrenExpanded);
			}

			const FOperatorStackEditorItem* DetailViewItem = InItem.GetDetailsViewItem().IsValid()
				? InItem.GetDetailsViewItem().Get()
				: CustomizeItem.Get();

			BodyDetailsView = CreateDetailsView(BodyCustomViewArgs, *DetailViewItem);

			return BodyDetailsView;
		}

		return nullptr;
	};

	// We are the root and we have multiple supported items selected
	if (!CustomizeItem.IsValid() && !Items.IsEmpty())
	{
		check(Items.Num() == 1)

		FOperatorStackEditorBodyBuilder StackBodyBuilder;
		StackCustomization->CustomizeStackBody(ItemTree, StackBodyBuilder);

		// Gather empty body text
		if (!StackBodyBuilder.GetEmptyBodyText().IsEmpty())
		{
			EmptyBodyDefaultText = StackBodyBuilder.GetEmptyBodyText();
		}

		// Check if we have a custom widget
		if (TSharedPtr<SWidget> StackWidget = StackBodyBuilder.GetCustomWidget())
		{
			return StackWidget;
		}

		// Scrollbar to be able to scroll in the full view per vertical stack
		TSharedPtr<SOperatorStackEditorStack> RootStack;
		TSharedRef<SScrollBox> ScrollBox = SNew(SScrollBox)
			.ScrollBarPadding(FMargin(0.f))
			.ScrollBarAlwaysVisible(true)
			.ScrollBarThickness(FVector2D(3.f))
			.ScrollBarVisibility(EVisibility::Hidden)
			.Orientation(Orient_Vertical)
			+ SScrollBox::Slot()
			.FillSize(1.f)
			.Padding(Padding)
			[
				SAssignNew(RootStack, SOperatorStackEditorStack, GetMainPanel(), StackCustomization, Items[0])
			];

		ItemsWidgets.Add(RootStack);

		return ScrollBox;
	}

	// We are not the root but we contain children then add a list view
	if (CustomizeItem.IsValid() && !Items.IsEmpty())
	{
		FOperatorStackEditorBodyBuilder ItemBodyBuilder;
		StackCustomization->CustomizeItemBody(CustomizeItem, ItemTree, ItemBodyBuilder);

		TSharedRef<SVerticalBox> ParentBox = SNew(SVerticalBox);

		if (TSharedPtr<SWidget> BodyWidget = GetItemWidget(ItemBodyBuilder))
		{
			ParentBox->AddSlot()
				.AutoHeight()
				.Padding(0.f)
				[
					BodyWidget.ToSharedRef()
				];
		}

		ParentBox->AddSlot()
			.FillHeight(1.f)
			.Padding(0.f)
			[
				SAssignNew(ItemsListView, SListView<FOperatorStackEditorItemPtr>)
				.ListViewStyle(&FOperatorStackEditorStyle::Get().GetWidgetStyle<FTableViewStyle>("ListViewStyle"))
				.ListItemsSource(&Items)
				.ClearSelectionOnClick(true)
				.SelectionMode(ESelectionMode::Multi)
				.OnKeyDownHandler(this, &SOperatorStackEditorStack::OnKeyDownHandler)
				.OnSelectionChanged(this, &SOperatorStackEditorStack::OnSelectionChanged)
				.OnGenerateRow(this, &SOperatorStackEditorStack::OnGenerateRow)
			];

		return SNew(SBox)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding(0.f)
			[
				ParentBox
			];
	}

	// we are a leaf, we have a parent but no children
	if (CustomizeItem.IsValid() && Items.IsEmpty())
	{
		FOperatorStackEditorBodyBuilder ItemBodyBuilder;
		StackCustomization->CustomizeItemBody(CustomizeItem, ItemTree, ItemBodyBuilder);

		// Gather empty body text
		if (!ItemBodyBuilder.GetEmptyBodyText().IsEmpty())
		{
			EmptyBodyDefaultText = ItemBodyBuilder.GetEmptyBodyText();
		}

		return GetItemWidget(ItemBodyBuilder);
	}

	// No children and no items, display empty body info text
	return SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.LineHeightPercentage(3.f)
			.Text(EmptyBodyDefaultText)
		];
}

TSharedPtr<SWidget> SOperatorStackEditorStack::GenerateFooterWidget()
{
	UOperatorStackEditorStackCustomization* StackCustomization = GetStackCustomization();
	const TSharedPtr<SOperatorStackEditorPanel> MainPanel = GetMainPanel();

	if (!CustomizeItem.IsValid())
	{
		return nullptr;
	}

	check(MainPanel.IsValid());

	const FOperatorStackEditorTree& ItemTree = MainPanel->GetItemTree(StackCustomization);

	FOperatorStackEditorFooterBuilder FooterBuilder;
	StackCustomization->CustomizeItemFooter(CustomizeItem, ItemTree, FooterBuilder);

	// We have set a custom widget for this footer
	if (TSharedPtr<SWidget> CustomWidget = FooterBuilder.GetCustomWidget())
	{
		return CustomWidget;
	}

	// Only build details view if we allow it
	if (FooterBuilder.GetShowDetailsView())
	{
		FCustomDetailsViewArgs FooterCustomViewArgs;
		FooterCustomViewArgs.bShowCategories = false;
		FooterCustomViewArgs.KeyframeHandler = MainPanel->GetKeyframeHandler();
		FooterCustomViewArgs.bAllowGlobalExtensions = true;
		FooterCustomViewArgs.ColumnSizeData = MainPanel->GetDetailColumnSize();

		for (const TSharedRef<FCustomDetailsViewItemId>& DetailsViewId : FooterBuilder.GetDisallowedDetailsViewItems())
		{
			FooterCustomViewArgs.ItemAllowList.Disallow(DetailsViewId.Get());
		}

		for (const TSharedRef<FCustomDetailsViewItemId>& DetailsViewId : FooterBuilder.GetAllowedDetailsViewItems())
		{
			FooterCustomViewArgs.ItemAllowList.Allow(DetailsViewId.Get());
		}

		for (const TSharedRef<FCustomDetailsViewItemId>& DetailsViewId : FooterBuilder.GetCollapsedDetailsViewItems())
		{
			FooterCustomViewArgs.ExpansionState.Add(DetailsViewId.Get(), ECustomDetailsViewExpansion::Collapsed);
		}

		for (const TSharedRef<FCustomDetailsViewItemId>& DetailsViewId : FooterBuilder.GetExpandedDetailsViewItems())
		{
			FooterCustomViewArgs.ExpansionState.Add(DetailsViewId.Get(), ECustomDetailsViewExpansion::SelfAndChildrenExpanded);
		}

		const FOperatorStackEditorItem* DetailViewItem = FooterBuilder.GetDetailsViewItem().IsValid()
			? FooterBuilder.GetDetailsViewItem().Get()
			: CustomizeItem.Get();

		FooterDetailsView = CreateDetailsView(FooterCustomViewArgs, *DetailViewItem);

		return FooterDetailsView;
	}

	return nullptr;
}

TSharedRef<SWidget> SOperatorStackEditorStack::GenerateMenuWidget(FName InMenuName) const
{
	UOperatorStackEditorMenuContext* const MenuContext = NewObject<UOperatorStackEditorMenuContext>();

	MenuContext->SetContext(GetContext());
	MenuContext->SetItem(CustomizeItem);

	const TSharedPtr<FExtender> MenuExtender;
	const FToolMenuContext ToolMenuContext(CommandList, MenuExtender, MenuContext);

	return UToolMenus::Get()->GenerateWidget(InMenuName, ToolMenuContext);
}

EVisibility SOperatorStackEditorStack::GetHeaderVisibility() const
{
	return !bHiddenByFilter ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SOperatorStackEditorStack::GetBodyVisibility() const
{
	return !bHiddenByFilter && bHeaderExpanded ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SOperatorStackEditorStack::GetFooterVisibility() const
{
	return !bHiddenByFilter && bHeaderExpanded ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SOperatorStackEditorStack::GetMessageBoxVisibility() const
{
	const EOperatorStackEditorMessageType MessageType = MessageBoxType.Get(EOperatorStackEditorMessageType::None);

	if (MessageType == EOperatorStackEditorMessageType::None)
	{
		return EVisibility::Collapsed;
	}

	const FText MessageText = MessageBoxText.Get(FText::GetEmpty());

	if (MessageText.IsEmpty())
	{
		return EVisibility::Collapsed;
	}

	return EVisibility::Visible;
}

FLinearColor SOperatorStackEditorStack::GetMessageBoxBackgroundColor() const
{
	return MessageBoxColors.FindRef(MessageBoxType.Get(EOperatorStackEditorMessageType::None));
}

const FSlateBrush* SOperatorStackEditorStack::GetMessageBoxIcon() const
{
	return MessageBoxIcons.FindRef(MessageBoxType.Get(EOperatorStackEditorMessageType::None));
}

EVisibility SOperatorStackEditorStack::GetMessageBoxIconVisibility() const
{
	return GetMessageBoxIcon() ? EVisibility::Visible : EVisibility::Collapsed;
}

void SOperatorStackEditorStack::RequestSearchAsync() const
{
	if (const TSharedPtr<SOperatorStackEditorPanel> MainPanel = GetMainPanel())
	{
		if (SearchBox.IsValid())
		{
			MainPanel->FilterItemsAsync(SearchBox->GetText());
		}
	}
}

void SOperatorStackEditorStack::OnSearchTextChanged(const FText& InSearchText)
{
	RequestSearchAsync();
}

void SOperatorStackEditorStack::FilterItems(const FText& InText)
{
	if (!SearchBox.IsValid())
	{
		return;
	}

	if (!SearchBox->GetText().EqualTo(InText))
	{
		SearchBox->SetText(InText);
	}

	const TSet<FString> SearchOR = SearchedKeywords;
	TSet<FString> SearchAND;

	const FString FilterString = InText.ToString();
	if (!FilterString.IsEmpty())
	{
		SearchAND.Add(FilterString);
	}

	HandleRecursiveSearch(SearchOR, SearchAND);
}

void SOperatorStackEditorStack::OnSearchTextCommitted(const FText& InFilterText, ETextCommit::Type InCommitType)
{
	if (InCommitType == ETextCommit::OnCleared)
	{
		const FText& EmptyText = FText::GetEmpty();
		SearchBox->SetText(EmptyText);
		OnSearchTextChanged(EmptyText);
		FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::Cleared);
	}
}

void SOperatorStackEditorStack::OnSearchPinnedKeyword(ECheckBoxState InCheckState, FString InPinnedKeyword)
{
	if (InCheckState == ECheckBoxState::Checked)
	{
		SearchedKeywords.Add(InPinnedKeyword);
	}
	else if (InCheckState == ECheckBoxState::Unchecked)
	{
		SearchedKeywords.Remove(InPinnedKeyword);
	}

	RequestSearchAsync();
}

bool SOperatorStackEditorStack::MatchSearch(const TSet<FString>& InSearchedKeywordsOR, const TSet<FString>& InSearchedKeywordsAND) const
{
	if (InSearchedKeywordsOR.IsEmpty() && InSearchedKeywordsAND.IsEmpty())
	{
		return true;
	}

	// Search in set for full keyword
	{
		bool bSearchMatched = false;
		if (!InSearchedKeywordsOR.IsEmpty())
		{
			bSearchMatched = !SearchableKeywords.Intersect(InSearchedKeywordsOR).IsEmpty();
		}

		if (!InSearchedKeywordsAND.IsEmpty())
		{
			bSearchMatched = SearchableKeywords.Includes(InSearchedKeywordsAND);
		}

		if (bSearchMatched)
		{
			return true;
		}
	}

	// Search by character for OR keywords
	bool bORSearchMatched = false;
	for (const FString& SearchedKeyword : InSearchedKeywordsOR)
	{
		for (const FString& Keyword : SearchableKeywords)
		{
			if (Keyword.Find(SearchedKeyword, ESearchCase::IgnoreCase, ESearchDir::FromStart) != INDEX_NONE)
			{
				bORSearchMatched = true;
				break;
			}
		}

		if (bORSearchMatched)
		{
			break;
		}
	}

	if (InSearchedKeywordsOR.IsEmpty())
	{
		bORSearchMatched = true;
	}

	// Search by character for AND keywords
	bool bANDSearchMatched = false;
	for (const FString& SearchedKeyword : InSearchedKeywordsAND)
	{
		for (const FString& Keyword : SearchableKeywords)
		{
			if (Keyword.Find(SearchedKeyword, ESearchCase::IgnoreCase, ESearchDir::FromStart) != INDEX_NONE)
			{
				bANDSearchMatched = true;
				break;
			}
		}

		if (!bANDSearchMatched)
		{
			break;
		}
	}

	if (InSearchedKeywordsAND.IsEmpty())
	{
		bANDSearchMatched = true;
	}

	return bORSearchMatched && bANDSearchMatched;
}

bool SOperatorStackEditorStack::HandleRecursiveSearch(const TSet<FString>& InSearchedKeywords_OR, const TSet<FString>& InSearchedKeywords_AND)
{
	bool bMatchSearch = false;

	for (const TSharedPtr<SOperatorStackEditorStack>& ItemWidget : ItemsWidgets)
	{
		if (ItemWidget.IsValid())
		{
			if (ItemWidget->HandleRecursiveSearch(InSearchedKeywords_OR, InSearchedKeywords_AND))
			{
				bMatchSearch = true;
			}
		}
	}

	// Do not hide root item if nothing was found
	if (CustomizeItem.IsValid() && CustomizeItem->HasValue())
	{
		// Search for properties row in details view
		bool bRowFound = false;

		if (BodyDetailsView)
		{
			bRowFound |= BodyDetailsView->FilterItems(InSearchedKeywords_AND.Array());
		}

		if (FooterDetailsView)
		{
			bRowFound |= FooterDetailsView->FilterItems(InSearchedKeywords_AND.Array());
		}

		if (!InSearchedKeywords_AND.IsEmpty())
		{
			bMatchSearch |= bRowFound;
		}
		
		bMatchSearch |= MatchSearch(InSearchedKeywords_OR, InSearchedKeywords_AND);
		bHiddenByFilter = !bMatchSearch;
	}

	return bMatchSearch;
}

TSharedRef<SWidget> SOperatorStackEditorStack::GenerateStackWidget()
{
	const TSharedRef<SVerticalBox> StackWidget = SNew(SVerticalBox);

	const TSharedPtr<SWidget> HeaderWidget = GenerateHeaderWidget();
	const TSharedPtr<SWidget> BodyWidget = GenerateBodyWidget();
	const TSharedPtr<SWidget> FooterWidget = GenerateFooterWidget();

	if (HeaderWidget.IsValid())
	{
		StackWidget->AddSlot()
		.AutoHeight()
		[
			HeaderWidget.ToSharedRef()
		];
	}

	if (BodyWidget.IsValid())
	{
		StackWidget->AddSlot()
		.FillHeight(1.f)
		[
			SNew(SBox)
			.Padding(0.f)
			.Visibility(this, &SOperatorStackEditorStack::GetBodyVisibility)
			[
				BodyWidget.ToSharedRef()
			]
		];
	}

	if (FooterWidget.IsValid())
	{
		StackWidget->AddSlot()
		.AutoHeight()
		[
			SNew(SBox)
			.Visibility(this, &SOperatorStackEditorStack::GetFooterVisibility)
			[
				FooterWidget.ToSharedRef()
			]
		];
	}

	// Only add border if we have a parent
	if (CustomizeItem.IsValid() && Items.IsEmpty())
	{
		return SNew(SHorizontalBox)
			.Visibility(this, &SOperatorStackEditorStack::GetHeaderVisibility)

			+ SHorizontalBox::Slot()
			.Padding(0.f)
			.AutoWidth()
			[
				SNew(SSeparator)
				.Visibility(BorderColor == FLinearColor::Transparent ? EVisibility::Collapsed : EVisibility::Visible)
				.ColorAndOpacity(BorderColor)
				.SeparatorImage(FAppStyle::GetBrush("ThinLine.Horizontal"))
				.Thickness(3.0f)
				.Orientation(EOrientation::Orient_Vertical)
			]

			+ SHorizontalBox::Slot()
			.Padding(0.f)
			.FillWidth(1.f)
			[
				SNew(SBox)
				.Padding(2.f)
				[
					SNew(SOverlay)
					+ SOverlay::Slot()
					[
						SNew(SColorBlock)
						.Color(FOperatorStackEditorStyle::Get().GetColor("ForegroundColor"))
					]
					+ SOverlay::Slot()
					[
						StackWidget
					]
				]
			];
	}

	return StackWidget;
}

void SOperatorStackEditorStack::OnHeaderExpansionChanged(bool bInExpansion)
{
	bHeaderExpanded = bInExpansion;

	if (const TSharedPtr<SOperatorStackEditorPanel> MainPanel = GetMainPanel())
	{
		if (CustomizeItem.IsValid())
		{
			MainPanel->SaveItemExpansionState(CustomizeItem->GetHash(), bHeaderExpanded);
		}
	}
}

void SOperatorStackEditorStack::OnSelectionChanged(FOperatorStackEditorItemPtr InItem, ESelectInfo::Type InSelect) const
{
	if (InSelect != ESelectInfo::Direct && !IsSelectableRow(InItem))
	{
		ItemsListView->SetItemSelection(InItem, false);
	}
}

FReply SOperatorStackEditorStack::OnKeyDownHandler(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
{
	bool bProcessed = false;

	if (ItemsListView.IsValid() && ItemsListView->GetNumItemsSelected() > 0)
	{
		for (FOperatorStackEditorItemPtr Item : ItemsListView->GetSelectedItems())
		{
			const int32 Idx = Items.Find(Item);

			if (Idx != INDEX_NONE && ItemsWidgets.IsValidIndex(Idx))
			{
				if (ItemsWidgets[Idx]->CommandList.IsValid())
				{
					bProcessed = ItemsWidgets[Idx]->CommandList->ProcessCommandBindings(InKeyEvent);
				}
			}
		}
	}
	else if (CustomizeItem.IsValid() && CommandList.IsValid())
	{
		bProcessed = CommandList->ProcessCommandBindings(InKeyEvent);
	}

	if (bProcessed)
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SOperatorStackEditorStack::OnHeaderMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// Open context menu
	if (CustomizeItem.IsValid() && MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		if (UToolMenus::Get()->IsMenuRegistered(ContextMenuName))
		{
			FSlateApplication::Get().PushMenu(
				AsShared(),
				FWidgetPath(),
				GenerateMenuWidget(ContextMenuName),
				MouseEvent.GetLastScreenSpacePosition(),
				FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
			);

			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

bool SOperatorStackEditorStack::IsSelectableRow(FOperatorStackEditorItemPtr InItem) const
{
	if (InItem.IsValid())
	{
		if (UOperatorStackEditorStackCustomization* StackCustomization = GetStackCustomization())
		{
			return StackCustomization->OnIsItemSelectable(InItem);
		}
	}

	return false;
}

TSharedRef<ICustomDetailsView> SOperatorStackEditorStack::CreateDetailsView(const FCustomDetailsViewArgs& InArgs, const FOperatorStackEditorItem& InItem)
{
	TSharedRef<ICustomDetailsView> CustomDetailsView = ICustomDetailsViewModule::Get().CreateCustomDetailsView(InArgs);

	if (InItem.GetValueType().GetTypeEnum() == EOperatorStackEditorItemType::Object)
	{
		if (InItem.GetValueCount() == 1)
		{
			UObject* Object = InItem.Get<UObject>(0);
			CustomDetailsView->SetObject(Object);
		}
		else
		{
			const TArray<UObject*> Objects = InItem.GetAsArray<UObject>();
			CustomDetailsView->SetObjects(Objects);
		}
	}
	else if (InItem.GetValueType().GetTypeEnum() == EOperatorStackEditorItemType::Struct)
	{
		// multi support for struct on scope is not yet available
		check(InItem.GetValueCount() == 1)

		const FOperatorStackEditorStructItem* Item = static_cast<const FOperatorStackEditorStructItem*>(&InItem);
		CustomDetailsView->SetStruct(Item->GetStructOnScope());
	}

	return CustomDetailsView;
}

#undef LOCTEXT_NAMESPACE
