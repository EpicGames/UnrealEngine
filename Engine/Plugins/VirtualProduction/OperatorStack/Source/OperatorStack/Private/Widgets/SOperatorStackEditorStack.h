// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Builders/OperatorStackEditorHeaderBuilder.h"
#include "Contexts/OperatorStackEditorContext.h"
#include "Items/OperatorStackEditorItem.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

enum class ECheckBoxState : uint8;
class FReply;
class FUICommandList;
class ICustomDetailsView;
class SOperatorStackEditorPanel;
class SSearchBox;
class UOperatorStackEditorStackCustomization;
struct EVisibility;
struct FButtonStyle;
struct FCustomDetailsViewArgs;

/** Represent a stack customization widget */
class SOperatorStackEditorStack : public SCompoundWidget
{
	friend class SOperatorStackEditorStackRow;

public:
	static constexpr float Padding = 2.f;
	static const TMap<EOperatorStackEditorMessageType, FLinearColor> MessageBoxColors;
	static const TMap<EOperatorStackEditorMessageType, const FSlateBrush*> MessageBoxIcons;

	SLATE_BEGIN_ARGS(SOperatorStackEditorStack) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs
		, const TSharedPtr<SOperatorStackEditorPanel> InMainPanel
		, TObjectPtr<UOperatorStackEditorStackCustomization> InCustomization
		, const FOperatorStackEditorItemPtr& InCustomizeItem);

	UOperatorStackEditorStackCustomization* GetStackCustomization() const
	{
		return StackCustomizationWeak.Get();
	}

	TSharedPtr<SOperatorStackEditorPanel> GetMainPanel() const
	{
		return MainPanelWeak.Pin();
	}

	FOperatorStackEditorContextPtr GetContext() const;

	/** Apply a filter on all items */
	void FilterItems(const FText& InText);

protected:
	TSharedRef<ITableRow> OnGenerateRow(FOperatorStackEditorItemPtr InItem, const TSharedRef<STableViewBase>& InOwnerTable);

	/** Key down handler for list */
	FReply OnKeyDownHandler(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent);

	/** Mouse event for header */
	FReply OnHeaderMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/** Header expansion state changed */
	void OnHeaderExpansionChanged(bool bInExpansion);

	/** List selection changed */
	void OnSelectionChanged(FOperatorStackEditorItemPtr InItem, ESelectInfo::Type InSelect) const;

	/** Is this item selectable in the list view */
	bool IsSelectableRow(FOperatorStackEditorItemPtr InItem) const;

	/** A stack widget consists of a header, body and footer */
	TSharedRef<SWidget> GenerateStackWidget();

	TSharedPtr<SWidget> GenerateHeaderWidget();
	TSharedPtr<SWidget> GenerateBodyWidget();
	TSharedPtr<SWidget> GenerateFooterWidget();

	/** Generate this named menu widget for this context */
	TSharedRef<SWidget> GenerateMenuWidget(FName InMenuName) const;

	EVisibility GetHeaderVisibility() const;
	EVisibility GetBodyVisibility() const;
	EVisibility GetFooterVisibility() const;

	EVisibility GetMessageBoxVisibility() const;
	FLinearColor GetMessageBoxBackgroundColor() const;
	const FSlateBrush* GetMessageBoxIcon() const;
	EVisibility GetMessageBoxIconVisibility() const;

	/** The customization to use */
	TWeakObjectPtr<UOperatorStackEditorStackCustomization> StackCustomizationWeak;

	/** The main panel we are in */
	TWeakPtr<SOperatorStackEditorPanel> MainPanelWeak;

	/** Customization item for this stack */
	FOperatorStackEditorItemPtr CustomizeItem;

	/** Children items of this item */
	TArray<FOperatorStackEditorItemPtr> Items;

	/** The list view if we contain any children */
	TSharedPtr<SListView<FOperatorStackEditorItemPtr>> ItemsListView;

	/** Header custom view for properties in header */
	TSharedPtr<ICustomDetailsView> HeaderDetailsView;

	/** Body custom view for properties in body */
	TSharedPtr<ICustomDetailsView> BodyDetailsView;

	/** Footer custom view for properties in footer */
	TSharedPtr<ICustomDetailsView> FooterDetailsView;

	/** Children items widget created from this, used for search propagation */
	TArray<TSharedPtr<SOperatorStackEditorStack>> ItemsWidgets;

	/** The search bar if any, only for root operator stack */
	TSharedPtr<SSearchBox> SearchBox;

	/** Contains pinned searched keywords, only for root operator stack */
	TSet<FString> SearchedKeywords;

	/** Contains searchable keywords for this item */
	TSet<FString> SearchableKeywords;

	/** Is header expanded to show body and footer */
	bool bHeaderExpanded = true;

	/** Color of the border around this stack */
	FLinearColor BorderColor = FLinearColor::Transparent;

	/** List of command available for this item */
	TSharedPtr<FUICommandList> CommandList;

	/** Name of the context menu */
	FName ContextMenuName;

	/** When set will display an alert message in the header */
	TAttribute<FText> MessageBoxText;

	/** When set will display an alert message in the header */
	TAttribute<EOperatorStackEditorMessageType> MessageBoxType;

	bool bHiddenByFilter = false;

private:
	static TSharedRef<ICustomDetailsView> CreateDetailsView(const FCustomDetailsViewArgs& InArgs, const FOperatorStackEditorItem& InItem);

	void OnSearchTextChanged(const FText& InSearchText);
	void OnSearchTextCommitted(const FText& InFilterText, ETextCommit::Type InCommitType);
	void OnSearchPinnedKeyword(ECheckBoxState InCheckState, FString InPinnedKeyword);

	/** Request an async search on the main panel */
	void RequestSearchAsync() const;

	/** Handles search in a recursive way, hiding items that do not match keywords */
	bool HandleRecursiveSearch(const TSet<FString>& InSearchedKeywords_OR, const TSet<FString>& InSearchedKeywords_AND);

	/** Returns true when this item matches the keywords */
	bool MatchSearch(const TSet<FString>& InSearchedKeywordsOR, const TSet<FString>& InSearchedKeywordsAND) const;
};

