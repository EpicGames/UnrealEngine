// Copyright Epic Games, Inc. All Rights Reserved.

#include "SOperatorStackExpanderButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"

void SOperatorStackExpanderButton::Construct(const FArguments& InArgs)
{
	bIsItemExpanded = InArgs._StartsExpanded;
	
	CollapsedButtonIcon = InArgs._CollapsedButtonIcon;
	ExpandedButtonIcon = InArgs._ExpandedButtonIcon;
	CollapsedHoveredButtonIcon = InArgs._CollapsedHoveredButtonIcon;
	ExpandedHoveredButtonIcon = InArgs._ExpandedHoveredButtonIcon;
	
	if (InArgs._OnExpansionStateChanged.IsBound())
	{
		OnExpansionStateChangedEvent = InArgs._OnExpansionStateChanged;
	}

	ChildSlot
	[
		SNew(SBox)
		.MinDesiredWidth(12)
		.MaxDesiredHeight(12)
		[
			SAssignNew(ExpanderButton, SButton)
			.ButtonStyle(FCoreStyle::Get(), "NoBorder")
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.ClickMethod(EButtonClickMethod::MouseDown)
			.OnClicked(this, &SOperatorStackExpanderButton::OnExpanderButtonClicked)
			.ContentPadding(0)
			.IsFocusable(false)
			[
				SNew(SImage)
				.Image(this, &SOperatorStackExpanderButton::GetExpanderIcon)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			]
		]
	];

	BroadcastExpansion();
}

const FSlateBrush* SOperatorStackExpanderButton::GetExpanderIcon() const
{
	if (bIsItemExpanded)
	{
		if (ExpanderButton->IsHovered())
		{
			return ExpandedHoveredButtonIcon;
		}
		
		return ExpandedButtonIcon;
	}

	if (ExpanderButton->IsHovered())
	{
		return CollapsedHoveredButtonIcon;
	}

	return CollapsedButtonIcon;
}

FReply SOperatorStackExpanderButton::OnExpanderButtonClicked()
{
	ToggleExpansion();

	return FReply::Handled();
}

void SOperatorStackExpanderButton::BroadcastExpansion() const
{
	if (OnExpansionStateChangedEvent.IsBound())
	{
		OnExpansionStateChangedEvent.Execute(bIsItemExpanded);
	}
}

void SOperatorStackExpanderButton::ToggleExpansion()
{
	bIsItemExpanded = !bIsItemExpanded;

	BroadcastExpansion();
}
