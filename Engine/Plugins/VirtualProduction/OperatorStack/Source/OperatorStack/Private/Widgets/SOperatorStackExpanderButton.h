// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/AppStyle.h"
#include "Styling/SlateBrush.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SButton;

DECLARE_DELEGATE_OneParam(FOperatorStackExpanderButtonStateChanged, bool)

/** A simple expander button with image, fires an event everytime state changes */
class SOperatorStackExpanderButton : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SOperatorStackExpanderButton)
		: _StartsExpanded(true)
		, _CollapsedButtonIcon(FAppStyle::Get().GetBrush(TEXT("TreeArrow_Collapsed")))
		, _ExpandedButtonIcon(FAppStyle::Get().GetBrush(TEXT("TreeArrow_Expanded")))
		, _CollapsedHoveredButtonIcon(FAppStyle::Get().GetBrush(TEXT("TreeArrow_Collapsed_Hovered")))
		, _ExpandedHoveredButtonIcon(FAppStyle::Get().GetBrush(TEXT("TreeArrow_Expanded_Hovered")))
	{}
	/** Are we expanded by default */
	SLATE_ARGUMENT(bool, StartsExpanded)
	/** Image to change the look of the button */
	SLATE_ARGUMENT(const FSlateBrush*, CollapsedButtonIcon)
	SLATE_ARGUMENT(const FSlateBrush*, ExpandedButtonIcon)
	SLATE_ARGUMENT(const FSlateBrush*, CollapsedHoveredButtonIcon)
	SLATE_ARGUMENT(const FSlateBrush*, ExpandedHoveredButtonIcon)
	/** Called when the expansion state changes */
	SLATE_EVENT(FOperatorStackExpanderButtonStateChanged, OnExpansionStateChanged)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs);

private:
	const FSlateBrush* GetExpanderIcon() const;
	FReply OnExpanderButtonClicked();
	
	void BroadcastExpansion() const;
	void ToggleExpansion();
	
	FOperatorStackExpanderButtonStateChanged OnExpansionStateChangedEvent;
	
	TSharedPtr<SButton> ExpanderButton;
	
	const FSlateBrush* CollapsedButtonIcon = nullptr;
	const FSlateBrush* ExpandedButtonIcon = nullptr;
	const FSlateBrush* CollapsedHoveredButtonIcon = nullptr;
	const FSlateBrush* ExpandedHoveredButtonIcon = nullptr;
	
	bool bIsItemExpanded = true;
};
