// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/TableRows/SOperatorStackEditorStackRow.h"

#include "Customizations/OperatorStackEditorStackCustomization.h"
#include "Framework/Application/SlateApplication.h"
#include "Styles/OperatorStackEditorStyle.h"
#include "Widgets/DragDropOps/OperatorStackEditorDragDropOp.h"
#include "Widgets/Layout/SSeparator.h"

void SOperatorStackEditorStackRow::Construct(const FArguments& InArgs
	, const TSharedRef<STableViewBase>& InOwnerTableView
	, const TSharedPtr<SOperatorStackEditorStack>& InOuterStack
	, const FOperatorStackEditorItemPtr& InCustomizeItem)
{
	OuterStackWeak = InOuterStack;

	check(InOuterStack->GetStackCustomization());

	UOperatorStackEditorStackCustomization* Customization = InOuterStack->GetStackCustomization();
	TSharedPtr<SOperatorStackEditorPanel> MainPanel = InOuterStack->MainPanelWeak.Pin();

	InnerStack = SNew(SOperatorStackEditorStack, MainPanel, Customization, InCustomizeItem);

	InOuterStack->ItemsWidgets.Add(InnerStack);

	ChildSlot
	[
		SNew(SVerticalBox)
		.Visibility(InnerStack.Get(), &SOperatorStackEditorStack::GetVisibility)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SSeparator)
			.ColorAndOpacity(FOperatorStackEditorStyle::Get().GetColor("BackgroundColor"))
			.SeparatorImage(FAppStyle::GetBrush("ThinLine.Horizontal"))
			.Thickness(SOperatorStackEditorStack::Padding)
			.Orientation(EOrientation::Orient_Horizontal)
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			InnerStack.ToSharedRef()
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SSeparator)
			.ColorAndOpacity(FOperatorStackEditorStyle::Get().GetColor("BackgroundColor"))
			.SeparatorImage(FAppStyle::GetBrush("ThinLine.Horizontal"))
			.Thickness(SOperatorStackEditorStack::Padding)
			.Orientation(EOrientation::Orient_Horizontal)
		]
	];

	STableRow<FOperatorStackEditorItemPtr>::ConstructInternal(
		STableRow<FOperatorStackEditorItemPtr>::FArguments()
			.ShowSelection(true)
			.OnDragDetected(this, &SOperatorStackEditorStackRow::OnStackDragDetected)
			.OnCanAcceptDrop(this, &SOperatorStackEditorStackRow::OnStackCanAcceptDrop)
			.OnDrop(this, &SOperatorStackEditorStackRow::OnStackDrop),
			InOwnerTableView
	);

	SetBorderImage(TAttribute<const FSlateBrush*>(this, &SOperatorStackEditorStackRow::GetBorder));
}

FReply SOperatorStackEditorStackRow::OnStackDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointer) const
{
	const TSharedPtr<SOperatorStackEditorStack> OuterStack = OuterStackWeak.Pin();

	if (OuterStack.IsValid() && OuterStack->ItemsListView.IsValid() && OuterStack->ItemsListView->GetNumItemsSelected() > 0)
	{
		UOperatorStackEditorStackCustomization* Customization = OuterStack->GetStackCustomization();
		TArray<TSharedPtr<FOperatorStackEditorItem>> SelectedItems;

		for (FOperatorStackEditorItemPtr SelectedItem : OuterStack->ItemsListView->GetSelectedItems())
		{
			if (SelectedItem.IsValid() && Customization->OnIsItemDraggable(SelectedItem))
			{
				SelectedItems.Add(SelectedItem);
			}
		}

		if (!SelectedItems.IsEmpty())
		{
			const TSharedRef<FOperatorStackEditorDragDropOp> StackDragDropOp = FOperatorStackEditorDragDropOp::New(SelectedItems);
			return FReply::Handled().BeginDragDrop(StackDragDropOp);
		}
	}

	return FReply::Unhandled();
}

TOptional<EItemDropZone> SOperatorStackEditorStackRow::OnStackCanAcceptDrop(const FDragDropEvent& InDropEvent
	, EItemDropZone InZone
	, FOperatorStackEditorItemPtr InZoneItem) const
{
	if (const TSharedPtr<FOperatorStackEditorDragDropOp> StackDragDropOp = InDropEvent.GetOperationAs<FOperatorStackEditorDragDropOp>())
	{
		if (InnerStack.IsValid())
		{
			UOperatorStackEditorStackCustomization* StackCustomization = InnerStack->GetStackCustomization();

			TOptional<EItemDropZone> DropZone = StackCustomization->OnItemCanAcceptDrop(StackDragDropOp->GetDraggedItems(), InZoneItem, InZone);
			StackDragDropOp->SetDropZone(DropZone);

			return DropZone;
		}
	}

	return TOptional<EItemDropZone>();
}

FReply SOperatorStackEditorStackRow::OnStackDrop(FDragDropEvent const& InEvent) const
{
	if (const TSharedPtr<FOperatorStackEditorDragDropOp> StackDragDropOp = InEvent.GetOperationAs<FOperatorStackEditorDragDropOp>())
	{
		const TOptional<EItemDropZone> DropZone = StackDragDropOp->GetDropZone();

		if (InnerStack.IsValid() && DropZone.IsSet())
		{
			UOperatorStackEditorStackCustomization* StackCustomization = InnerStack->GetStackCustomization();
			const FOperatorStackEditorItemPtr ZoneItem = InnerStack->CustomizeItem;

			StackCustomization->OnDropItem(StackDragDropOp->GetDraggedItems(), ZoneItem, DropZone.GetValue());
		}
	}

	return FReply::Handled().EndDragDrop();
}

int32 SOperatorStackEditorStackRow::OnPaintDropIndicator(EItemDropZone InZone, const FPaintArgs& InPaintArgs, const FGeometry& InGeometry, const FSlateRect& InSlateRect, FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle, bool bInParentEnabled) const
{
	if (const TSharedPtr<FDragDropOperation> DragDropOperation = FSlateApplication::Get().GetDragDroppingContent())
	{
		if (DragDropOperation->IsOfType<FOperatorStackEditorDragDropOp>())
		{
			const FSlateBrush* DropIndicatorBrush = GetDropIndicatorBrush(InZone);

			constexpr float OffsetX = 10.0f;
			const FVector2D Offset(OffsetX * GetIndentLevel(), 0.f);
			FSlateDrawElement::MakeBox
			(
				OutDrawElements,
				InLayerId++,
				InGeometry.ToPaintGeometry(FVector2D(InGeometry.GetLocalSize() - Offset), FSlateLayoutTransform(Offset)),
				DropIndicatorBrush,
				ESlateDrawEffect::None,
				DropIndicatorBrush->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
			);
		}
	}

	return InLayerId;
}
