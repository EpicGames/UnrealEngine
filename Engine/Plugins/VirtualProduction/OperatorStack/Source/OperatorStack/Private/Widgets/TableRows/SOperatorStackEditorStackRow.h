// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Items/OperatorStackEditorItem.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SOperatorStackEditorStack.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"

/** A single row item in the stack listview that contains a stack, get it it's recursive :) */
class SOperatorStackEditorStackRow : public STableRow<FOperatorStackEditorItemPtr>
{

public:
	SLATE_BEGIN_ARGS(SOperatorStackEditorStackRow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs
		, const TSharedRef<STableViewBase>& InOwnerTableView
		, const TSharedPtr<SOperatorStackEditorStack>& InOuterStack
		, const FOperatorStackEditorItemPtr& InCustomizeItem);

	TSharedPtr<SOperatorStackEditorStack> GetInnerStack() const
	{
		return InnerStack;
	}

protected:
	/** When starting a stack drag operation */
	FReply OnStackDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointer) const;

	/** Whether we allow or disallow drop */
	TOptional<EItemDropZone> OnStackCanAcceptDrop(const FDragDropEvent& InDropEvent, EItemDropZone InZone, FOperatorStackEditorItemPtr InZoneItem) const;

	/** When a drop event happens on an item */
	FReply OnStackDrop(FDragDropEvent const& InEvent) const;

	/** Draw indicator for drop */
	virtual int32 OnPaintDropIndicator(EItemDropZone InZone, const FPaintArgs& InPaintArgs, const FGeometry& InGeometry, const FSlateRect& InSlateRect, FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle, bool bInParentEnabled) const override;

	TWeakPtr<SOperatorStackEditorStack> OuterStackWeak;
	TSharedPtr<SOperatorStackEditorStack> InnerStack;
};
