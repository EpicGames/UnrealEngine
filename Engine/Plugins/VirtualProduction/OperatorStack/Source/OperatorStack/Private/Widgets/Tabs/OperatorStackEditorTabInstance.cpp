// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Tabs/OperatorStackEditorTabInstance.h"

#include "Algo/Find.h"
#include "Contexts/OperatorStackEditorContext.h"
#include "Customizations/OperatorStackEditorStackCustomization.h"
#include "Editor.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "GameFramework/Actor.h"
#include "ILevelEditor.h"
#include "Items/OperatorStackEditorItem.h"
#include "Items/OperatorStackEditorObjectItem.h"
#include "Selection.h"
#include "Subsystems/OperatorStackEditorSubsystem.h"
#include "Templates/Projection.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SOperatorStackEditorWidget.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "OperatorStackEditorTabInstance"

FOperatorStackEditorTabInstance::FOperatorStackEditorTabInstance(const TSharedRef<ILevelEditor>& InLevelEditor)
	: LevelEditorWeak(InLevelEditor)
{
}

FOperatorStackEditorTabInstance::~FOperatorStackEditorTabInstance()
{
	UnbindDelegates();
	UnregisterTab();
}

TSharedPtr<SDockTab> FOperatorStackEditorTabInstance::InvokeTab()
{
	const TSharedPtr<ILevelEditor> LevelEditor = LevelEditorWeak.Pin();

	if (!LevelEditor.IsValid())
	{
		return nullptr;
	}

	const TSharedPtr<FTabManager> TabManager = LevelEditor->GetTabManager();

	if (!TabManager.IsValid())
	{
		return nullptr;
	}

	TSharedPtr<SDockTab> Tab = TabManager->TryInvokeTab(UOperatorStackEditorSubsystem::TabId);

	if (Tab.IsValid())
	{
		Tab->ActivateInParent(ETabActivationCause::SetDirectly);
		Tab->DrawAttention();
	}

	return Tab;
}

bool FOperatorStackEditorTabInstance::CloseTab()
{
	const TSharedPtr<ILevelEditor> LevelEditor = LevelEditorWeak.Pin();

	if (!LevelEditor.IsValid())
	{
		return false;
	}

	const TSharedPtr<FTabManager> TabManager = LevelEditor->GetTabManager();

	if (!TabManager.IsValid())
	{
		return false;
	}

	const TSharedPtr<SDockTab> Tab = TabManager->FindExistingLiveTab(UOperatorStackEditorSubsystem::TabId);

	if (Tab.IsValid())
	{
		return Tab->RequestCloseTab();
	}

	return false;
}

bool FOperatorStackEditorTabInstance::RefreshTab(UObject* InContext, bool bInForce)
{
	const TSharedPtr<ILevelEditor> LevelEditor = LevelEditorWeak.Pin();
	if (!LevelEditor.IsValid())
	{
		return false;
	}

	const UTypedElementSelectionSet* SelectionSet = LevelEditor->GetElementSelectionSet();
	if (!SelectionSet)
	{
		return false;
	}

	const UWorld* ContextWorld = InContext->GetWorld();
	if (LevelEditor->GetWorld() != ContextWorld)
	{
		return false;
	}

	if (bInForce)
	{
		OnSelectionSetChanged(SelectionSet, bInForce);
	}
	else
	{
		const TSharedPtr<SOperatorStackEditorWidget> Widget = GetOperatorStackEditorWidget();
		if (!Widget.IsValid())
		{
			return false;
		}

		const FOperatorStackEditorContextPtr Context = Widget->GetContext();
		if (!Context.IsValid())
		{
			return false;
		}

		const AActor* ContextActor = InContext->GetTypedOuter<AActor>();
		if (!ContextActor)
		{
			return false;
		}

		const TArray<UObject*> SelectedObjects = SelectionSet->GetSelectedObjects();
		if (!SelectedObjects.Contains(ContextActor) && !SelectedObjects.Contains(InContext))
		{
			return false;
		}

		Widget->RefreshContext();
	}
	
	return true;
}

bool FOperatorStackEditorTabInstance::FocusTab(const UObject* InContext, FName InIdentifier)
{
	const TSharedPtr<ILevelEditor> LevelEditor = LevelEditorWeak.Pin();
	if (!LevelEditor.IsValid())
	{
		return false;
	}

	if (LevelEditor->GetWorld() != InContext->GetWorld())
	{
		return false;
	}

	InvokeTab();

	if (!InIdentifier.IsNone())
	{
		if (const TSharedPtr<SOperatorStackEditorWidget> Widget = GetOperatorStackEditorWidget())
		{
			Widget->SetActiveCustomization(InIdentifier);
		}
	}

	return true;
}

TSharedPtr<SOperatorStackEditorWidget> FOperatorStackEditorTabInstance::GetOperatorStackEditorWidget() const
{
	TSharedPtr<SOperatorStackEditorWidget> Widget;

	if (WidgetIdentifier == INDEX_NONE)
	{
		return Widget;
	}

	if (UOperatorStackEditorSubsystem* Subsystem = UOperatorStackEditorSubsystem::Get())
	{
		Widget = Subsystem->FindWidget(WidgetIdentifier);
	}

	return Widget;
}

bool FOperatorStackEditorTabInstance::RegisterTab()
{
	const TSharedPtr<ILevelEditor> LevelEditor = LevelEditorWeak.Pin();

	if (!LevelEditor.IsValid())
	{
		return false;
	}

	const TSharedPtr<FTabManager> TabManager = LevelEditor->GetTabManager();

	if (!TabManager.IsValid())
	{
		return false;
	}

	static const FText DisplayNameText = LOCTEXT("OperatorStackTabName", "Operator Stack");

	TSharedRef<FWorkspaceItem> Category = WorkspaceMenu::GetMenuStructure().GetLevelEditorCategory();

	// Group Operator Stack under Motion Design > Motion Design Tools if the plugin is enabled.
	constexpr auto GroupProjection = Projection(&TSharedRef<FWorkspaceItem>::Get, &FWorkspaceItem::GetFName);
	if (const TSharedRef<FWorkspaceItem>* RootGroup = Algo::FindBy(Category->GetChildItems(), FName("MotionDesignGroup"), GroupProjection))
	{
		if (const TSharedRef<FWorkspaceItem>* ToolsGroup = Algo::FindBy((*RootGroup)->GetChildItems(), FName("MotionDesignToolsGroup"), GroupProjection))
		{
			Category = *ToolsGroup;
		}
	}

	TabManager->RegisterTabSpawner(
		UOperatorStackEditorSubsystem::TabId,
		FOnSpawnTab::CreateSP(this, &FOperatorStackEditorTabInstance::OnSpawnTab)
	)
	.SetDisplayNameAttribute(DisplayNameText)
	.SetDisplayName(DisplayNameText)
	.SetGroup(Category)
	.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.UserDefinedStruct"));

	// To auto-invoke tab
	UnbindDelegates();
	BindDelegates();

	return true;
}

TSharedRef<SDockTab> FOperatorStackEditorTabInstance::OnSpawnTab(const FSpawnTabArgs& InArgs)
{
	const TSharedPtr<ILevelEditor> LevelEditor = LevelEditorWeak.Pin();
	UOperatorStackEditorSubsystem* Subsystem = UOperatorStackEditorSubsystem::Get();

	check(!!Subsystem)
	check(LevelEditor.IsValid())

	const TSharedRef<SOperatorStackEditorWidget> Widget = Subsystem->GenerateWidget();
	WidgetIdentifier = Widget->GetPanelId();

	constexpr bool bForce = false;
	OnSelectionSetChanged(LevelEditor->GetElementSelectionSet(), bForce);

	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSPLambda(this, [this](float)
	{
		const UOperatorStackEditorSubsystem* Subsystem = UOperatorStackEditorSubsystem::Get();
		const TSharedPtr<SOperatorStackEditorWidget> Widget = GetOperatorStackEditorWidget();
		
		if (Subsystem && Widget)
		{
			Subsystem->OnOperatorStackSpawnedDelegate.Broadcast(Widget.ToSharedRef());
		}

		return false;
	}), 0);

	return SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		.Content()
		[
			Widget
		];
}

void FOperatorStackEditorTabInstance::OnSelectionSetChanged(const UTypedElementSelectionSet* InSelection, bool bInForce)
{
	if (!InSelection || !LevelEditorWeak.IsValid())
	{
		return;
	}

	// Cancel previous request
	if (SelectionDelegateHandle.IsValid())
	{
		FTSTicker::RemoveTicker(SelectionDelegateHandle);
		SelectionDelegateHandle.Reset();
	}

	// Wait one tick before processing since selection event can do : Select New -> Deselect Previous to avoid handling Previous again
	TWeakObjectPtr<const UTypedElementSelectionSet> SelectionWeak(InSelection);
	SelectionDelegateHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSP(this, &FOperatorStackEditorTabInstance::CheckActiveSelection, SelectionWeak, bInForce));
}

void FOperatorStackEditorTabInstance::OnSelectionChanged(UObject* InSelectionObject)
{
	if (const USelection* Selection = Cast<USelection>(InSelectionObject))
	{
		constexpr bool bForce = false;
		OnSelectionSetChanged(Selection->GetElementSelectionSet(), bForce);
	}
}

bool FOperatorStackEditorTabInstance::CheckActiveSelection(float, TWeakObjectPtr<const UTypedElementSelectionSet> InSelectionWeak, bool bInForce)
{
	const UTypedElementSelectionSet* Selection = InSelectionWeak.Get();

	if (!GEditor || !Selection || IsEngineExitRequested())
	{
		return false; // Stop
	}

	TSet<UObject*> SelectedObjects(Selection->GetSelectedObjects());

	if (USelection* ActorSelection = GEditor->GetSelectedActors())
	{
		TArray<UObject*> Actors;
		ActorSelection->GetSelectedObjects(Actors);
		SelectedObjects.Append(Actors);
	}

	if (USelection* ComponentSelection = GEditor->GetSelectedComponents())
	{
		TArray<UObject*> Components;
		ComponentSelection->GetSelectedObjects(Components);
		SelectedObjects.Append(Components);
	}

	if (USelection* ObjectSelection = GEditor->GetSelectedObjects())
	{
		TArray<UObject*> Objects;
		ObjectSelection->GetSelectedObjects(Objects);
		SelectedObjects.Append(Objects);
	}

	TSharedPtr<SOperatorStackEditorWidget> Widget = GetOperatorStackEditorWidget();

	TSharedPtr<ILevelEditor> LevelEditor = LevelEditorWeak.Pin();
	TArray<FOperatorStackEditorItemPtr> SelectedItems;
	Algo::TransformIf(
		SelectedObjects,
		SelectedItems,
		[LevelEditor](const UObject* InObject)
		{
			return InObject && InObject->GetWorld() == LevelEditor->GetWorld();
		},
		[](UObject* InObject)
		{
			return MakeShared<FOperatorStackEditorObjectItem>(InObject);
		});

	if (!Widget.IsValid() && SelectedItems.IsEmpty())
	{
		return false; // Stop
	}

	const FOperatorStackEditorContext NewContext(SelectedItems);

	if (!bInForce
		&& Widget.IsValid()
		&& Widget->GetContext()
		&& *Widget->GetContext().Get() == NewContext)
	{
		return false; // Stop
	}

	if (!Widget.IsValid())
	{
		// Invoke tab
		const UOperatorStackEditorSubsystem* Subsystem = UOperatorStackEditorSubsystem::Get();

		check(!!Subsystem)

		bool bInvokeTab = false;
		Subsystem->ForEachCustomization([this, &NewContext, &bInvokeTab](UOperatorStackEditorStackCustomization* InCustomization)->bool
		{
			if (InCustomization && InCustomization->ShouldFocusCustomization(NewContext))
			{
				bInvokeTab = true;
				return false;
			}

			return true;
		});

		if (bInvokeTab)
		{
			InvokeTab();
			Widget = GetOperatorStackEditorWidget();
		}
	}

	if (Widget.IsValid())
	{
		Widget->SetContext(NewContext);
	}

	return false; // Stop
}

bool FOperatorStackEditorTabInstance::UnregisterTab()
{
	const TSharedPtr<ILevelEditor> LevelEditor = LevelEditorWeak.Pin();

	if (!LevelEditor.IsValid())
	{
		return false;
	}

	const TSharedPtr<FTabManager> TabManager = LevelEditor->GetTabManager();

	if (!TabManager.IsValid())
	{
		return false;
	}

	return TabManager->UnregisterTabSpawner(UOperatorStackEditorSubsystem::TabId);
}

void FOperatorStackEditorTabInstance::BindDelegates()
{
	const TSharedPtr<ILevelEditor> LevelEditor = LevelEditorWeak.Pin();

	if (!LevelEditor.IsValid())
	{
		return;
	}

	constexpr bool bForce = false;
	LevelEditor->GetMutableElementSelectionSet()->OnChanged().AddSP(this, &FOperatorStackEditorTabInstance::OnSelectionSetChanged, bForce);
	USelection::SelectionChangedEvent.AddSP(this, &FOperatorStackEditorTabInstance::OnSelectionChanged);
}

void FOperatorStackEditorTabInstance::UnbindDelegates()
{
	const TSharedPtr<ILevelEditor> LevelEditor = LevelEditorWeak.Pin();

	if (!LevelEditor.IsValid())
	{
		return;
	}

	LevelEditor->GetMutableElementSelectionSet()->OnChanged().RemoveAll(this);
	USelection::SelectionChangedEvent.RemoveAll(this);
}

#undef LOCTEXT_NAMESPACE