// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class FSpawnTabArgs;
class ILevelEditor;
class SDockTab;
class SOperatorStackEditorWidget;
class UObject;
class UTypedElementSelectionSet;

struct FOperatorStackEditorTabInstance : TSharedFromThis<FOperatorStackEditorTabInstance>
{
	explicit FOperatorStackEditorTabInstance(const TSharedRef<ILevelEditor>& InLevelEditor);
	virtual ~FOperatorStackEditorTabInstance();

	TSharedPtr<SDockTab> InvokeTab();
	bool CloseTab();
	bool RefreshTab(UObject* InContext, bool bInForce);
	bool FocusTab(const UObject* InContext, FName InIdentifier);

	TSharedPtr<SOperatorStackEditorWidget> GetOperatorStackEditorWidget() const;
	bool RegisterTab();
	bool UnregisterTab();

	TSharedPtr<ILevelEditor> GetLevelEditor() const
	{
		return LevelEditorWeak.Pin();
	}

private:
	void BindDelegates();
	void UnbindDelegates();

	TSharedRef<SDockTab> OnSpawnTab(const FSpawnTabArgs& InArgs);
	void OnSelectionSetChanged(const UTypedElementSelectionSet* InSelection, bool bInForce);
	void OnSelectionChanged(UObject* InSelectionObject);

	bool CheckActiveSelection(float, TWeakObjectPtr<const UTypedElementSelectionSet> InSelectionWeak, bool bInForce);

	TWeakPtr<ILevelEditor> LevelEditorWeak;
	int32 WidgetIdentifier = INDEX_NONE;
	FTSTicker::FDelegateHandle SelectionDelegateHandle;
};
