// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

class FCustomDetailsViewItemId;
class SWidget;
struct FOperatorStackEditorItem;

/** Body builder for item */
struct OPERATORSTACKEDITOR_API FOperatorStackEditorBodyBuilder
{
	/** Override the stack generated widget with a custom one */
	FOperatorStackEditorBodyBuilder& SetCustomWidget(TSharedPtr<SWidget> InWidget);

	/** Show a details view for the current item */
	FOperatorStackEditorBodyBuilder& SetShowDetailsView(bool bInShowDetailsView);

	/** Set the item that needs to be displayed by the details view overriding current item */
	FOperatorStackEditorBodyBuilder& SetDetailsViewItem(TSharedPtr<FOperatorStackEditorItem> InItem);

	/** Disallow specific property in the details view */
	FOperatorStackEditorBodyBuilder& DisallowProperty(FProperty* InProperty);

	/** Disallow specific category in the details view */
	FOperatorStackEditorBodyBuilder& DisallowCategory(const FName& InCategory);

	/** Allow specific property in the details view */
	FOperatorStackEditorBodyBuilder& AllowProperty(FProperty* InProperty);

	/** Allow specific category in the details view */
	FOperatorStackEditorBodyBuilder& AllowCategory(const FName& InCategory);

	/** Expand property state in the details view */
	FOperatorStackEditorBodyBuilder& ExpandProperty(FProperty* InProperty);

	/** Collapse property state in the details view */
	FOperatorStackEditorBodyBuilder& CollapseProperty(FProperty* InProperty);

	/** Set a text that will be displayed when it is empty */
	FOperatorStackEditorBodyBuilder& SetEmptyBodyText(const FText& InText);

	TSharedPtr<SWidget> GetCustomWidget() const
	{
		return CustomWidget;
	}

	bool GetShowDetailsView() const
	{
		return bShowDetailsView;
	}

	TSharedPtr<FOperatorStackEditorItem> GetDetailsViewItem() const
	{
		return DetailsViewItem;
	}

	const TArray<TSharedRef<FCustomDetailsViewItemId>>& GetDisallowedDetailsViewItems() const
	{
		return DisallowedDetailsViewItems;
	}

	const TArray<TSharedRef<FCustomDetailsViewItemId>>& GetAllowedDetailsViewItems() const
	{
		return AllowedDetailsViewItems;
	}

	const TArray<TSharedRef<FCustomDetailsViewItemId>>& GetCollapsedDetailsViewItems() const
	{
		return CollapsedDetailsViewItems;
	}

	const TArray<TSharedRef<FCustomDetailsViewItemId>>& GetExpandedDetailsViewItems() const
	{
		return ExpandedDetailsViewItems;
	}

	const FText& GetEmptyBodyText() const
	{
		return EmptyBodyText;
	}

protected:
	/** Custom widget to replace content */
	TSharedPtr<SWidget> CustomWidget = nullptr;

	/** Does this body contains a details view */
	bool bShowDetailsView = false;

	/** Override actual item to display detail view */
	TSharedPtr<FOperatorStackEditorItem> DetailsViewItem = nullptr;

	/** When the body is empty, this will be displayed */
	FText EmptyBodyText;

	/** Disallowed items inside details view */
	TArray<TSharedRef<FCustomDetailsViewItemId>> DisallowedDetailsViewItems;

	/** Allowed items inside details view */
	TArray<TSharedRef<FCustomDetailsViewItemId>> AllowedDetailsViewItems;

	/** Collapsed items inside details view */
	TArray<TSharedRef<FCustomDetailsViewItemId>> CollapsedDetailsViewItems;

	/** Expanded items inside details view */
	TArray<TSharedRef<FCustomDetailsViewItemId>> ExpandedDetailsViewItems;
};