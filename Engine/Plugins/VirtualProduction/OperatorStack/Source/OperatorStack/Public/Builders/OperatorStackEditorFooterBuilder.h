// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OperatorStackEditorBodyBuilder.h"

/** Body and Footer are the same at the moment, we may add additional behaviour here */
typedef FOperatorStackEditorBodyBuilder FOperatorStackEditorFooterBuilder;