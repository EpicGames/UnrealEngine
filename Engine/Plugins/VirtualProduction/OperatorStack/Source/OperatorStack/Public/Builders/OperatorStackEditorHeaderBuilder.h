// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Styling/SlateBrush.h"
#include "UObject/Object.h"

class FCustomDetailsViewItemId;
class FUICommandList;
class SWidget;

enum class EOperatorStackEditorMessageType
{
	None,
	/** Blue message for information message */
	Info,
	/** Green message for success message */
	Success,
	/** Yellow message for warning message */
	Warning,
	/** Red message for error message */
	Error
};

/** Header builder for item */
struct OPERATORSTACKEDITOR_API FOperatorStackEditorHeaderBuilder
{
	/** Set the icon to display for this item header */
	FOperatorStackEditorHeaderBuilder& SetIcon(const FSlateBrush* InIcon);

	/** Set the label to display for this item header */
	FOperatorStackEditorHeaderBuilder& SetLabel(const FText& InLabel);

	/** Set the tooltip to display for this item header */
	FOperatorStackEditorHeaderBuilder& SetTooltip(const FText& InTooltip);

	/** Sets the border color surrounding the item */
	FOperatorStackEditorHeaderBuilder& SetBorderColor(const FLinearColor& InColor);

	/** Set a single property to expose in the header, like a boolean */
	FOperatorStackEditorHeaderBuilder& SetProperty(FProperty* InProperty);

	/** Set a small menu that contains icon entries on the right */
	FOperatorStackEditorHeaderBuilder& SetToolbarMenu(const FName& InActionMenuName);

	/** Set a menu that appears when we press a button with a label and icon in the header */
	FOperatorStackEditorHeaderBuilder& SetToolMenu(const FName& InToolMenuName, const FText& InToolMenuLabel, const FSlateBrush* InToolMenuIcon);

	/** Allow search on this item, if the stack header is searchable a search box will appear */
	FOperatorStackEditorHeaderBuilder& SetSearchAllowed(bool bInSearchable);

	/** Sets keywords for this item, if the keyword is contained inside this item will show */
	FOperatorStackEditorHeaderBuilder& SetSearchKeywords(const TSet<FString>& InKeywords);

	/** Sets pinned keywords on root item, they will appear on top of the search bar */
	FOperatorStackEditorHeaderBuilder& SetSearchPinnedKeywords(const TSet<FString>& InKeywords);

	/** Whether the item is expandable */
	FOperatorStackEditorHeaderBuilder& SetExpandable(bool bInExpandable);

	/** Whether the item starts expanded or collapsed */
	FOperatorStackEditorHeaderBuilder& SetStartsExpanded(bool bInExpanded);

	/** Whether the item is draggable */
	FOperatorStackEditorHeaderBuilder& SetDraggable(bool bInDraggable);

	/** Sets available commands on this item */
	FOperatorStackEditorHeaderBuilder& SetCommandList(TSharedPtr<FUICommandList> InCommands);

	/** Sets context menu name on this item */
	FOperatorStackEditorHeaderBuilder& SetContextMenu(const FName& InToolMenuName);

	/** Sets the custom widget replacing the stack generated widget */
	FOperatorStackEditorHeaderBuilder& SetCustomWidget(TSharedPtr<SWidget> InWidget);

	/** Sets whether to show a message box in the header for warning, error, success operations */
	FOperatorStackEditorHeaderBuilder& SetMessageBox(const TAttribute<EOperatorStackEditorMessageType>& InType, const TAttribute<FText>& InText);

	const FSlateBrush* GetIcon() const
	{
		return Icon;
	}

	const FText& GetLabel() const
	{
		return Label;
	}

	const FText& GetTooltip() const
	{
		return Tooltip;
	}

	const FLinearColor& GetBorderColor() const
	{
		return BorderColor;
	}

	const FName& GetToolbarMenuName() const
	{
		return ToolbarMenuName;
	}

	const TSharedPtr<FCustomDetailsViewItemId>& GetProperty() const
	{
		return Property;
	}

	const FName& GetToolMenuName() const
	{
		return ToolMenuName;
	}

	const FSlateBrush* GetToolMenuIcon() const
	{
		return ToolMenuIcon;
	}

	const FText& GetToolMenuLabel() const
	{
		return ToolMenuLabel;
	}

	bool GetSearchAllowed() const
	{
		return bSearchAllowed;
	}

	const TSet<FString>& GetSearchKeywords() const
	{
		return SearchKeywords;
	}

	const TSet<FString>& GetSearchPinnedKeywords() const
	{
		return SearchPinnedKeywords;
	}

	bool GetExpandable() const
	{
		return bExpandable;
	}

	bool GetStartsExpanded() const
	{
		return bStartsExpanded;
	}

	bool GetDraggable() const
	{
		return bDraggable;
	}

	const TSharedPtr<FUICommandList>& GetCommandList() const
	{
		return CommandList;
	}

	const FName& GetContextMenuName() const
	{
		return ContextMenuName;
	}

	TSharedPtr<SWidget> GetCustomWidget() const
	{
		return CustomWidget;
	}

	const TAttribute<EOperatorStackEditorMessageType>& GetMessageBoxType() const
	{
		return MessageBoxType;
	}

	const TAttribute<FText>& GetMessageBoxText() const
	{
		return MessageBoxText;
	}

protected:
	/** On the left */
	const FSlateBrush* Icon = nullptr;

	/** On the left, after to icon */
	FText Label;

	/** Shown on the header */
	FText Tooltip;

	/** The border color surrounding the item */
	FLinearColor BorderColor = FLinearColor::Transparent;

	/** Menu on the right before property, should not contain a lot of entries and only icons */
	FName ToolbarMenuName;

	/** On the right, property id to expose in the header */
	TSharedPtr<FCustomDetailsViewItemId> Property = nullptr;

	/** Menu present in the header for additional actions */
	FName ToolMenuName;

	/** Menu icon in the header */
	const FSlateBrush* ToolMenuIcon = nullptr;

	/** Menu label in the header as a tooltip and label */
	FText ToolMenuLabel;

	/** Allow use of search bar to filter content */
	bool bSearchAllowed = false;

	/** Keywords to test against search input */
	TSet<FString> SearchKeywords;

	/** Pinned keywords for search, will appear below search box, only for root stack */
	TSet<FString> SearchPinnedKeywords;

	/** Allow expandable header to show the body and footer */
	bool bExpandable = false;

	/** Starts expanded or not */
	bool bStartsExpanded = true;

	/** Allow drag operation on this item */
	bool bDraggable = false;

	/** Commands executable for this item */
	TSharedPtr<FUICommandList> CommandList;

	/** Context menu available on this item */
	FName ContextMenuName;

	/** Widget that overrides the whole stack generated widget */
	TSharedPtr<SWidget> CustomWidget;

	/** The type of message box we want to show */
	TAttribute<EOperatorStackEditorMessageType> MessageBoxType;

	/** The text content of the message box */
	TAttribute<FText> MessageBoxText;
};