// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Items/OperatorStackEditorItem.h"
#include "Templates/SharedPointer.h"

/** Represent the current context we customize for the whole operator stack */
struct FOperatorStackEditorContext final
{
	FOperatorStackEditorContext() = default;
	explicit FOperatorStackEditorContext(const TArray<TSharedPtr<FOperatorStackEditorItem>>& InItems)
		: Items(InItems)
	{
		CachedHash = 0;

		TSet<uint32> Hashes;
		Hashes.Reserve(InItems.Num());
		for (const TSharedPtr<FOperatorStackEditorItem>& Item : InItems)
		{
			if (Item.IsValid())
			{
				const uint32 ItemHash = Item->GetHash();
				bool bIsAlreadyInSet = false;
				Hashes.Add(ItemHash, &bIsAlreadyInSet);

				if (!bIsAlreadyInSet)
				{
					CachedHash += ItemHash;
				}
			}
		}
	}

	/** Items we want to customize */
	TConstArrayView<TSharedPtr<FOperatorStackEditorItem>> GetItems() const
	{
		return Items;
	}
	
	friend uint32 GetTypeHash(const FOperatorStackEditorContext& InItem)
	{
		return InItem.CachedHash;
	}

	bool operator==(const FOperatorStackEditorContext& InOther) const
	{
		return CachedHash == InOther.CachedHash;
	}

protected:
	/** Context items being customized */
	TArray<TSharedPtr<FOperatorStackEditorItem>> Items;

	/** Context items hash to compare contexts */
	uint32 CachedHash = 0;
};

typedef TSharedPtr<FOperatorStackEditorContext> FOperatorStackEditorContextPtr;
typedef TWeakPtr<FOperatorStackEditorContext> FOperatorStackEditorContextPtrWeak;