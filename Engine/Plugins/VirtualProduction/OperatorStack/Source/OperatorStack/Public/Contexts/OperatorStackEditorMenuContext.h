// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "OperatorStackEditorMenuContext.generated.h"

struct FOperatorStackEditorContext;
struct FOperatorStackEditorItem;

/** Context passed in UToolMenu when generating entries with selected items */
UCLASS(MinimalAPI)
class UOperatorStackEditorMenuContext : public UObject
{
	GENERATED_BODY()

public:
	TSharedPtr<FOperatorStackEditorContext> GetContext() const
	{
		return ContextWeak.Pin();
	}

	void SetContext(TSharedPtr<FOperatorStackEditorContext> InContext)
	{
		ContextWeak = InContext;
	}

	TSharedPtr<FOperatorStackEditorItem> GetItem() const
	{
		return ItemWeak.Pin();
	}

	void SetItem(TSharedPtr<FOperatorStackEditorItem> InItem)
	{
		ItemWeak = InItem;
	}

protected:
	/** The current context this menu is extending */
	TWeakPtr<FOperatorStackEditorContext> ContextWeak;

	/** The item this menu should apply to */
	TWeakPtr<FOperatorStackEditorItem> ItemWeak;
};