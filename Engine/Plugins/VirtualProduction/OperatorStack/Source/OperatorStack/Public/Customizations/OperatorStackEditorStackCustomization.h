// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Builders/OperatorStackEditorBodyBuilder.h"
#include "Builders/OperatorStackEditorFooterBuilder.h"
#include "Builders/OperatorStackEditorHeaderBuilder.h"
#include "Items/OperatorStackEditorItem.h"
#include "Items/OperatorStackEditorTree.h"
#include "Misc/Optional.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "OperatorStackEditorStackCustomization.generated.h"

enum class EItemDropZone;
class UOperatorStackEditorStackRowCustomization;

/**
	Abstract class to represent an operator stack containing items,
	an item is represented by a header-body-footer,
	An item can contain multiple items (recursive),
	Children class extending this class are automatically registered
*/
UCLASS(MinimalAPI, Abstract, Transient)
class UOperatorStackEditorStackCustomization : public UObject
{
	GENERATED_BODY()

	friend class UOperatorStackEditorSubsystem;
	friend class SOperatorStackEditorStack;

public:
	UOperatorStackEditorStackCustomization()
		: UOperatorStackEditorStackCustomization(NAME_None, FText::GetEmpty())
	{}

	UOperatorStackEditorStackCustomization(const FName& InIdentifier, const FText& InLabel, int32 InPriority = INDEX_NONE)
		: Identifier(InIdentifier)
		, Label(InLabel)
		, Priority(InPriority)
	{}

	/** Registers items with this definition that can use this customization */
	OPERATORSTACKEDITOR_API bool RegisterCustomizationFor(const UStruct* InItemDefinition);
	OPERATORSTACKEDITOR_API bool RegisterCustomizationFor(const FFieldClass* InItemDefinition);

	/** Unregisters item definition for this customization */
	OPERATORSTACKEDITOR_API bool UnregisterCustomizationFor(const UStruct* InItemDefinition);
	OPERATORSTACKEDITOR_API bool UnregisterCustomizationFor(const FFieldClass* InItemDefinition);

	/** Checks if this customization is supported for this item */
	bool IsCustomizationSupportedFor(const FOperatorStackEditorItemPtr& InItem) const;

	/** Retrieves root supported item out of the provided context, Children classes must implement this */
	virtual bool GetRootItem(const FOperatorStackEditorContext& InContext, FOperatorStackEditorItemPtr& OutRootItem) const
	{
		return OutRootItem.IsValid() && OutRootItem->HasValue();
	}

	/** Retrieves the supported children items based out of a parent item, Children classes must implement this */
	virtual bool GetChildrenItem(const FOperatorStackEditorItemPtr& InItem, TArray<FOperatorStackEditorItemPtr>& OutChildrenItems) const
	{
		return !OutChildrenItems.IsEmpty();
	}

	/** Customize the header for a context */
	virtual void CustomizeStackHeader(const FOperatorStackEditorTree& InItemTree, FOperatorStackEditorHeaderBuilder& InStackHeaderBuilder) {}

	/** Customize the full stack body */
	virtual void CustomizeStackBody(const FOperatorStackEditorTree& InItemTree, FOperatorStackEditorBodyBuilder& InStackBodyBuilder) {}

	/** Customize the header for a supported item */
	virtual void CustomizeItemHeader(const FOperatorStackEditorItemPtr& InItem, const FOperatorStackEditorTree& InItemTree, FOperatorStackEditorHeaderBuilder& InHeaderBuilder) {}

	/** Customize the body for a supported item */
	virtual void CustomizeItemBody(const FOperatorStackEditorItemPtr& InItem, const FOperatorStackEditorTree& InItemTree, FOperatorStackEditorBodyBuilder& InBodyBuilder) {}

	/** Customize the body for a supported item */
	virtual void CustomizeItemFooter(const FOperatorStackEditorItemPtr& InItem, const FOperatorStackEditorTree& InItemTree, FOperatorStackEditorFooterBuilder& InFooterBuilder) {}

	/** Can an item be selected */
	virtual bool OnIsItemSelectable(const FOperatorStackEditorItemPtr& InItem)
	{
		return true;
	}

	/** Can an item be dragged */
	virtual bool OnIsItemDraggable(const FOperatorStackEditorItemPtr& InItem)
	{
		return false;
	}

	/** Get the valid drop zone of draggable items onto a zone item */
	virtual TOptional<EItemDropZone> OnItemCanAcceptDrop(const TArray<FOperatorStackEditorItemPtr>& InDraggedItems, const FOperatorStackEditorItemPtr& InTargetItem, EItemDropZone InTargetZone)
	{
		return TOptional<EItemDropZone>();
	}

	/** Handle dropped items onto target zone */
	virtual void OnDropItem(const TArray<FOperatorStackEditorItemPtr>& InDraggedItems, const FOperatorStackEditorItemPtr& InTargetItem, EItemDropZone InTargetZone) {}

	/** Get identifier of this customization */
	const FName& GetIdentifier() const
	{
		return Identifier;
	}

	/** Get display label of this customization */
	const FText& GetLabel() const
	{
		return Label;
	}

	/** Get stack priority in the toolbar */
	int32 GetPriority() const
	{
		return Priority;
	}

	/** Get the displayed icon of this customization */
	OPERATORSTACKEDITOR_API virtual const FSlateBrush* GetIcon() const;
	
	/** Refresh the selection linked to this context */
	OPERATORSTACKEDITOR_API void RefreshActiveSelection(UObject* InContext, bool bInForce) const;

	/** Switches section of operator stack and select this customization for this context */
	OPERATORSTACKEDITOR_API void FocusCustomization(const UObject* InContext) const;

	/** Whether new context should focus on this customization in the widget */
	virtual bool ShouldFocusCustomization(const FOperatorStackEditorContext& InContext) const
	{
		return false;
	};

private:
	/** Unique identifier of this customization */
	UPROPERTY()
	FName Identifier;

	/** Label displayed at the top to switch between customizations */
	UPROPERTY()
	FText Label;

	/** Priority for this stack in toolbar, highest numbers will result in placement before lowest number */
	UPROPERTY()
	int32 Priority;

	/** Supported definition for this customization stack */
	UPROPERTY()
	TSet<TObjectPtr<const UStruct>> SupportedDefinitions;

	/** Supported definition for this customization stack */
	TSet<const FFieldClass*> SupportedFieldClasses;
};