// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OperatorStackEditorItem.h"

/** Groups item of the same type together */
struct FOperatorStackEditorGroupItem : FOperatorStackEditorItem
{
	explicit FOperatorStackEditorGroupItem(const TArray<FOperatorStackEditorItemPtr>& InItems, const FOperatorStackEditorItemType& InType)
		: FOperatorStackEditorItem(
			InType
		)
	{
		CachedHash = 0;

		TSet<uint32> Hashes;
		Hashes.Reserve(InItems.Num());
		for (const FOperatorStackEditorItemPtr& Item : InItems)
		{
			if (Item.IsValid())
			{
				// No groups into groups
				check(Item->GetValueCount() == 1)

				// Group is only allowed for item of the same type
				check(GetValueType() == Item->GetValueType())

				const uint32 ItemHash = Item->GetHash();
				if (!Hashes.Contains(ItemHash))
				{
					CachedHash += ItemHash;
					Items.Add(Item);
					Hashes.Add(ItemHash);
				}
			}
		}
	}

	virtual uint32 GetValueCount() const override
	{
		return Items.Num();
	}

	virtual bool HasValue(uint32 InIndex) const override
	{
		if (Items.IsValidIndex(InIndex) && Items[InIndex].IsValid())
		{
			// No groups allowed so pick first item
			return Items[InIndex]->HasValue(0);
		}

		return FOperatorStackEditorItem::HasValue(InIndex);
	}

	virtual uint32 GetHash() const override
	{
		return CachedHash;
	}

	virtual void* GetValuePtr(uint32 InIndex) const override
	{
		if (Items.IsValidIndex(InIndex) && Items[InIndex].IsValid())
		{
			// No groups allowed so pick first item
			return Items[InIndex]->GetValuePtr(0);
		}

		return FOperatorStackEditorItem::GetValuePtr(InIndex);
	}

protected:
	TArray<FOperatorStackEditorItemPtr> Items;
};