// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/BaseStructureProvider.h"
#include "Items/OperatorStackEditorItemType.h"
#include "Templates/IsClass.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"

/** Abstract base parent item */
struct FOperatorStackEditorItem
{
	explicit FOperatorStackEditorItem(const FOperatorStackEditorItemType& InType)
		: ItemType(InType)
	{}

	virtual ~FOperatorStackEditorItem() = default;

	/**
	 * You can use this to check if Item.IsA<UObject>() or Item.IsA<FMyStruct>() or Item.IsA<FIntProperty>()
	 */
	template<typename InValueType>
	bool IsA() const
	{
		// For cases like IsA<UScriptStruct>()
		if constexpr (TIsDerivedFrom<InValueType, UStruct>::Value)
		{
			if (const UStruct* ValueStruct = ItemType.Get<UStruct>())
			{
				return ValueStruct->IsA<InValueType>();
			}
		}

		// Check if its a UObject derived class
		if constexpr (TIsDerivedFrom<InValueType, UObject>::Value)
		{
			const UStruct* ValueStruct = InValueType::StaticClass();
			return ItemType.IsChildOf<UStruct>(ValueStruct);
		}
		// Check if its a struct with StaticStruct()
		else if constexpr (TModels<CStaticStructProvider, InValueType>::Value)
		{
			const UStruct* ValueStruct = InValueType::StaticStruct();
			return ItemType.IsChildOf<UStruct>(ValueStruct);
		}
		// Check if base struct without StaticStruct()
		else if constexpr (TModels<CBaseStructureProvider, InValueType>::Value)
		{
			const UStruct* ValueStruct = TBaseStructure<InValueType>::Get();
			return ItemType.IsChildOf<UStruct>(ValueStruct);
		}
		// Check if its a FProperty derived class
		else if constexpr (TIsDerivedFrom<InValueType, FProperty>::Value)
		{
			const FFieldClass* ValueClass = InValueType::StaticClass();
			return ItemType.IsChildOf<FFieldClass>(ValueClass);
		}

		else
		{
			return false;
		}
	}

	/**
	 * You can use this to get the item Item.Get<UObject>() or Item.Get<FMyStruct>() or Item.Get<int32>()
	 */
	template <typename InValueType>
	InValueType* Get(uint32 InIndex) const
	{
		check(InIndex < GetValueCount())

		if (!HasValue(InIndex))
		{
			return nullptr;
		}

		if constexpr (TIsDerivedFrom<InValueType, UObject>::Value)
		{
			if (ItemType.GetTypeEnum() == EOperatorStackEditorItemType::Object)
			{
				return Cast<InValueType>(static_cast<UObject*>(GetValuePtr(InIndex)));
			}
		}
		else if constexpr (TModels_V<CStaticStructProvider, InValueType> || TModels_V<CBaseStructureProvider, InValueType>)
		{
			if (ItemType.GetTypeEnum() == EOperatorStackEditorItemType::Struct)
			{
				return static_cast<InValueType*>(GetValuePtr(InIndex));
			}
		}
		else if constexpr (TIsPODType<InValueType>::Value)
		{
			if (ItemType.GetTypeEnum() == EOperatorStackEditorItemType::Primitive)
			{
				return static_cast<InValueType*>(GetValuePtr(InIndex));
			}
		}

		return nullptr;
	}

	/**
	 * You can use this to get all the underlying items as an array of a specific type
	 */
	template <typename InValueType>
	TArray<InValueType*> GetAsArray() const
	{
		uint32 ValueCount = GetValueCount();

		TArray<InValueType*> Values;
		Values.Reserve(ValueCount);

		for (uint32 Index = 0; Index < ValueCount; Index++)
		{
			if (InValueType* Value = Get<InValueType>(Index))
			{
				Values.Add(Value);
			}
		}

		return Values;
	}

	/** Get the value type of this item */
	const FOperatorStackEditorItemType& GetValueType() const
	{
		return ItemType;
	}

	/** Get the amount of value stored within this item */
	virtual uint32 GetValueCount() const
	{
		return 0;
	}

	/** Checks if this item has at least one value and that it is usable */
	bool HasValue() const
	{
		bool bHasValue = false;

		for (uint32 Index = 0; Index < GetValueCount(); Index++)
		{
			bHasValue |= HasValue(Index);
		}

		return bHasValue;
	}

	/** Checks if this item has a value and that it is usable */
	virtual bool HasValue(uint32 InIndex) const
	{
		return false;
	}

	/** Get raw ptr to value, prefer using Get<>() instead */
	virtual void* GetValuePtr(uint32 InIndex) const
	{
		return nullptr;
	}

	/** Override in child to be able to compare item */
	virtual uint32 GetHash() const
	{
		return CachedHash;
	}

	friend uint32 GetTypeHash(const FOperatorStackEditorItem& InItem)
	{
		return InItem.GetHash();
	}

	bool operator==(const FOperatorStackEditorItem& InOtherItem) const
	{
		const uint32 ThisHash = GetHash();
		const uint32 OtherHash = InOtherItem.GetHash();

		return ThisHash != 0
			&& OtherHash != 0
			&& ThisHash == OtherHash;
	}

protected:
	uint32 CachedHash = 0;
	FOperatorStackEditorItemType ItemType;
};

typedef TSharedPtr<FOperatorStackEditorItem> FOperatorStackEditorItemPtr;
typedef TWeakPtr<FOperatorStackEditorItem> FOperatorStackEditorItemPtrWeak;
