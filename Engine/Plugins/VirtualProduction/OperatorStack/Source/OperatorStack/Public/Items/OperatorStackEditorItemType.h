// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

#include "Containers/Union.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"

/** Used to quickly get the general type of an item without casting */
enum class EOperatorStackEditorItemType : uint8
{
	None,
	Object,
	Struct,
	Primitive
};

/** Item type */
struct FOperatorStackEditorItemType
{
	template<typename InTypeClass
		UE_REQUIRES(std::is_base_of_v<UStruct, InTypeClass> || std::is_base_of_v<FFieldClass, InTypeClass>)>
	explicit FOperatorStackEditorItemType(const InTypeClass* InType, EOperatorStackEditorItemType InTypeEnum)
		: TypeClass(InType)
		, TypeEnum(InTypeEnum)
	{
	}

	template<typename InTypeClass
		UE_REQUIRES(std::is_base_of_v<UStruct, InTypeClass> || std::is_base_of_v<FFieldClass, InTypeClass>)>
	bool IsChildOf(const InTypeClass* InOtherType) const
	{
		if (TypeClass.HasSubtype<const InTypeClass*>())
		{
			return TypeClass.GetSubtype<const InTypeClass*>()->IsChildOf(InOtherType);
		}

		return false;
	}

	bool IsChildOf(const FOperatorStackEditorItemType* InOtherType) const
	{
		if (!InOtherType)
		{
			return false;
		}

		if (TypeClass.GetCurrentSubtypeIndex() != InOtherType->TypeClass.GetCurrentSubtypeIndex())
		{
			return false;
		}

		if (TypeClass.GetCurrentSubtypeIndex() == 0)
		{
			return TypeClass.GetSubtype<const UStruct*>()->IsChildOf(InOtherType->TypeClass.GetSubtype<const UStruct*>());
		}

		if (TypeClass.GetCurrentSubtypeIndex() == 1)
		{
			return TypeClass.GetSubtype<const FFieldClass*>()->IsChildOf(InOtherType->TypeClass.GetSubtype<const FFieldClass*>());
		}

		return false;
	}

	template<typename InTypeClass
		UE_REQUIRES(std::is_base_of_v<UStruct, InTypeClass> || std::is_base_of_v<FFieldClass, InTypeClass>)>
	const InTypeClass* Get() const
	{
		return TypeClass.HasSubtype<const InTypeClass*>() ? TypeClass.GetSubtype<const InTypeClass*>() : nullptr;
	}

	EOperatorStackEditorItemType GetTypeEnum() const
	{
		return TypeEnum;
	}

	friend uint32 GetTypeHash(const FOperatorStackEditorItemType& InItem)
	{
		uint32 TypeClassHash = 0;

		if (InItem.TypeClass.GetCurrentSubtypeIndex() == 0)
		{
			TypeClassHash = GetTypeHash(InItem.TypeClass.GetSubtype<const UStruct*>());
		}

		if (InItem.TypeClass.GetCurrentSubtypeIndex() == 1)
		{
			TypeClassHash = GetTypeHash(InItem.TypeClass.GetSubtype<const FFieldClass*>());
		}

		return HashCombine(TypeClassHash, static_cast<uint32>(InItem.TypeEnum));
	}

	bool operator==(const FOperatorStackEditorItemType& InOtherItem) const
	{
		return TypeEnum == InOtherItem.TypeEnum 
			&& TypeClass.GetCurrentSubtypeIndex() == InOtherItem.TypeClass.GetCurrentSubtypeIndex()
			&& ((TypeClass.GetCurrentSubtypeIndex() == 0 && TypeClass.GetSubtype<const UStruct*>() == InOtherItem.TypeClass.GetSubtype<const UStruct*>())
			|| (TypeClass.GetCurrentSubtypeIndex() == 1 && TypeClass.GetSubtype<const FFieldClass*>() == InOtherItem.TypeClass.GetSubtype<const FFieldClass*>()));
	}

private:
	TUnion<const UStruct*, const FFieldClass*> TypeClass;
	EOperatorStackEditorItemType TypeEnum = EOperatorStackEditorItemType::None;
};