// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OperatorStackEditorItem.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"

/** Object version of item */
struct FOperatorStackEditorObjectItem : FOperatorStackEditorItem
{
	template<
		typename InValueType
		UE_REQUIRES(TIsDerivedFrom<InValueType, UObject>::Value)>
	explicit FOperatorStackEditorObjectItem(InValueType* InItem)
		: FOperatorStackEditorItem(
			FOperatorStackEditorItemType(InItem ? InItem->GetClass() : nullptr, EOperatorStackEditorItemType::Object)
		)
	{
		ObjectWeak = InItem;
		CachedHash = GetTypeHash(InItem);
	}

	virtual uint32 GetValueCount() const override
	{
		return 1;
	}

	virtual bool HasValue(uint32 InIndex) const override
	{
		return ObjectWeak.IsValid();
	}

	virtual uint32 GetHash() const override
	{
		return CachedHash;
	}

	virtual void* GetValuePtr(uint32 InIndex) const override
	{
		return ObjectWeak.Get();
	}

protected:
	TWeakObjectPtr<UObject> ObjectWeak;
};