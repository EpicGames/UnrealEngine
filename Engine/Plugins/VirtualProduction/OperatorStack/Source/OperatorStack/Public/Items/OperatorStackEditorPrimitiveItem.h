// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OperatorStackEditorItem.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"

/** Primitive version of item */
struct FOperatorStackEditorPrimitiveItem : FOperatorStackEditorItem
{
	template<
		typename InItemDefinition
		UE_REQUIRES(TIsPODType<InItemDefinition>::Value)>
	explicit FOperatorStackEditorPrimitiveItem(FFieldClass* InFieldClass, InItemDefinition& InValue)
		: FOperatorStackEditorItem(
			FOperatorStackEditorItemType(InFieldClass, EOperatorStackEditorItemType::Primitive)
		)
	{
		MemoryPtr = reinterpret_cast<uint8*>(&InValue);
		CachedHash = GetTypeHash(InValue);
	}

	virtual uint32 GetValueCount() const override
	{
		return 1;
	}

	virtual bool HasValue(uint32 InIndex) const override
	{
		return MemoryPtr != nullptr;
	}

	virtual uint32 GetHash() const override
	{
		return CachedHash;
	}

	virtual void* GetValuePtr(uint32 InIndex) const override
	{
		return MemoryPtr;
	}

protected:
	uint8* MemoryPtr = nullptr;
};