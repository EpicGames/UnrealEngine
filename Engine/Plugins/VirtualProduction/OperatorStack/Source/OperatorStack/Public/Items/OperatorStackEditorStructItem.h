// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OperatorStackEditorItem.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/StructOnScope.h"
#include "UObject/UnrealType.h"

/** Struct version of item */
struct FOperatorStackEditorStructItem : FOperatorStackEditorItem
{
	// Version for UStruct
	template <typename InValueType
		UE_REQUIRES(TModels<CStaticStructProvider, InValueType>::Value)>
	explicit FOperatorStackEditorStructItem(InValueType& InStruct)
		: FOperatorStackEditorItem(
			FOperatorStackEditorItemType(InValueType::StaticStruct(), EOperatorStackEditorItemType::Struct)
		)
	{
		StructWeak = MakeShared<FStructOnScope>(InValueType::StaticStruct(), reinterpret_cast<uint8*>(&InStruct));

		if constexpr (TModels<CGetTypeHashable, InValueType>::Value)
		{
			CachedHash = GetTypeHash(InStruct);
		}
	}

	// Version for base struct
	template <typename InValueType
		UE_REQUIRES(TModels<CBaseStructureProvider, InValueType>::Value)>
	explicit FOperatorStackEditorStructItem(InValueType& InStruct)
		: FOperatorStackEditorItem(
			FOperatorStackEditorItemType(TBaseStructure<InValueType>::Get(), EOperatorStackEditorItemType::Struct)
		)
	{
		StructWeak = MakeShared<FStructOnScope>(TBaseStructure<InValueType>::Get(), reinterpret_cast<uint8*>(&InStruct));

		if constexpr (TModels<CGetTypeHashable, InValueType>::Value)
		{
			CachedHash = GetTypeHash(InStruct);
		}
	}

	// Version for struct on scope
	explicit FOperatorStackEditorStructItem(const TSharedPtr<FStructOnScope>& InStruct)
		: FOperatorStackEditorItem(
			FOperatorStackEditorItemType(InStruct.IsValid() ? InStruct->GetStruct() : nullptr, EOperatorStackEditorItemType::Struct)
		)
	{
		StructWeak = InStruct;

		if (InStruct.IsValid())
		{
			CachedHash = HashCombine(GetTypeHash(ItemType), GetTypeHash(*InStruct->GetStructMemory()));
		}
	}

	TSharedPtr<FStructOnScope> GetStructOnScope() const
	{
		return StructWeak.Pin();
	}

	virtual uint32 GetValueCount() const override
	{
		return 1;
	}

	virtual bool HasValue(uint32 InIndex) const override
	{
		if (const TSharedPtr<FStructOnScope> Struct = StructWeak.Pin())
		{
			return Struct->IsValid();
		}

		return false;
	}

	virtual uint32 GetHash() const override
	{
		return CachedHash;
	}

	virtual void* GetValuePtr(uint32 InIndex) const override
	{
		if (const TSharedPtr<FStructOnScope> Struct = StructWeak.Pin())
		{
			return Struct->GetStructMemory();
		}

		return nullptr;
	}

protected:
	TWeakPtr<FStructOnScope> StructWeak;
};