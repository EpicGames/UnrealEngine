// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Contexts/OperatorStackEditorContext.h"
#include "OperatorStackEditorItem.h"

class UOperatorStackEditorStackCustomization;

/** Customization tree used to find all supported items by a customization object */
struct FOperatorStackEditorTree
{
	explicit FOperatorStackEditorTree(UOperatorStackEditorStackCustomization* InCustomization, FOperatorStackEditorContextPtr InContext);

	/** Get all top items supported by this customization */
	OPERATORSTACKEDITOR_API FOperatorStackEditorItemPtr GetRootItem() const;

	/** Get supported children items from a supported item */
	TArray<FOperatorStackEditorItemPtr> GetChildrenItems(FOperatorStackEditorItemPtr InItem) const;

	/** Get supported parent item from a supported child item */
	FOperatorStackEditorItemPtr GetParentItem(FOperatorStackEditorItemPtr InItem) const;

	/** Get all leaf supported item */
	TArray<FOperatorStackEditorItemPtr> GetLeafItems() const;

	/** Get all supported items by this customization */
	OPERATORSTACKEDITOR_API TConstArrayView<FOperatorStackEditorItemPtr> GetAllItems() const;

	/** Get source context used to build this tree */
	OPERATORSTACKEDITOR_API const FOperatorStackEditorContext& GetContext() const;

	/** Get the customization used to build this tree */
	UOperatorStackEditorStackCustomization* GetCustomization() const;

	/** Checks if an item is already contained in the tree */
	bool Contains(FOperatorStackEditorItemPtr InItem) const;

private:
	/** Customization tree node used only internally by the tree */
	struct FOperatorStackEditorTreeNode
	{
		explicit FOperatorStackEditorTreeNode(int32 InItemIndex, int32 InParentIndex);

		int32 ItemIndex = INDEX_NONE;
		int32 ParentIndex = INDEX_NONE;
		TArray<int32> ChildrenIndices;
	};

	using FOperatorStackEditorTreeNodePtr = TSharedPtr<FOperatorStackEditorTreeNode>;

	TArray<FOperatorStackEditorItemPtr> GetSupportedChildrenItems(const FOperatorStackEditorItemPtr& InParentItem) const;
	void BuildTreeInternal(const TArray<FOperatorStackEditorItemPtr>& InItems, int32 InParent);

	/** All items in this tree */
	TArray<FOperatorStackEditorItemPtr> Items;

	/** All nodes in this tree */
	TArray<FOperatorStackEditorTreeNodePtr> Nodes;

	/** Root items in the tree */
	FOperatorStackEditorTreeNodePtr RootNode;

	/** Customization used to build this tree */
	TWeakObjectPtr<UOperatorStackEditorStackCustomization> CustomizationWeak;

	/** Source context used to build this tree */
	TWeakPtr<FOperatorStackEditorContext> ContextWeak;
};
