// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "Templates/SubclassOf.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "OperatorStackEditorSubsystem.generated.h"

class ILevelEditor;
class SOperatorStackEditorWidget;
class UObject;
class UOperatorStackEditorStackCustomization;
struct FOperatorStackEditorTabInstance;

/** Subsystem that handles operator stack customization */
UCLASS()
class UOperatorStackEditorSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

	friend class SOperatorStackEditorPanel;
	friend struct FOperatorStackEditorTabInstance;

public:
	OPERATORSTACKEDITOR_API static const FName TabId;
	OPERATORSTACKEDITOR_API static UOperatorStackEditorSubsystem* Get();
	
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnOperatorStackSpawned, TSharedRef<SOperatorStackEditorWidget> /** InWidget */)

	static FOnOperatorStackSpawned::RegistrationType& OnOperatorStackSpawned()
	{
		return OnOperatorStackSpawnedDelegate;
	}
	
	//~ Begin USubsystem
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	//~ End USubsystem

	OPERATORSTACKEDITOR_API bool RegisterStackCustomization(TSubclassOf<UOperatorStackEditorStackCustomization> InStackCustomizationClass);
	OPERATORSTACKEDITOR_API bool UnregisterStackCustomization(TSubclassOf<UOperatorStackEditorStackCustomization> InStackCustomizationClass);

	/** Loops through each customization stack in priority order, stops if false returned within function */
	bool ForEachCustomization(TFunctionRef<bool(UOperatorStackEditorStackCustomization*)> InFunction) const;

	/** Get customization with identifier */
	UOperatorStackEditorStackCustomization* GetCustomization(const FName& InName) const;

	/** Generate an operator stack widget */
	OPERATORSTACKEDITOR_API TSharedRef<SOperatorStackEditorWidget> GenerateWidget();

	/** Finds an existing operator stack widget */
	TSharedPtr<SOperatorStackEditorWidget> FindWidget(int32 InId);

	/** Finds existing operator stack widgets within a provided context */
	TArray<TSharedPtr<SOperatorStackEditorWidget>> FindWidgets(const UWorld* InContext);

	/** Loops through each operator stack widgets available */
	OPERATORSTACKEDITOR_API bool ForEachCustomizationWidget(TFunctionRef<bool(TSharedRef<SOperatorStackEditorWidget>)> InFunction) const;

	/** Refresh only selection of widget with specific context */
	OPERATORSTACKEDITOR_API void RefreshCustomizationWidget(UObject* InContext, bool bInForce);
	
	/** Focus customization widget with specific context */
	OPERATORSTACKEDITOR_API void FocusCustomizationWidget(const UObject* InContext, FName InIdentifier);

protected:
	OPERATORSTACKEDITOR_API static FOnOperatorStackSpawned OnOperatorStackSpawnedDelegate;

    void ScanForStackCustomizations();

	void OnWidgetDestroyed(int32 InPanelId);

	void OnLevelEditorCreated(TSharedPtr<ILevelEditor> InLevelEditor);

	/** Map of identifier and stack customization models */
	UPROPERTY()
	TMap<FName, TObjectPtr<UOperatorStackEditorStackCustomization>> CustomizationStacks;

	/** Current widget created associated to an identifier */
	TMap<int32, TWeakPtr<SOperatorStackEditorWidget>> CustomizationWidgets;

	/** Current tab within editor */
	TArray<TSharedPtr<FOperatorStackEditorTabInstance>> TabInstances;
};
