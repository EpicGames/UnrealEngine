// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class FDetailColumnSizeData;
class IDetailKeyframeHandler;
class UOperatorStackEditorStackCustomization;
struct FOperatorStackEditorContext;

/** Main operator stack widget created, only exposed function */
class SOperatorStackEditorWidget : public SCompoundWidget
{
public:
	/** Set the active customization in the panel */
	virtual void SetActiveCustomization(const FName& InCustomization) {}

	/** Sets the customization to show in the toolbar, overrides order */
	virtual void SetToolbarCustomizations(const TArray<FName>& InCustomizations) {}

	/** Get all the customizations object active on this widget */
	virtual void GetToolbarCustomizations(TArray<UOperatorStackEditorStackCustomization*>& OutCustomizations) {}

	/** Show the customization in the panel */
	virtual void ShowToolbarCustomization(FName InCustomization) {}

	/** Hide the customization from the panel */
	virtual void HideToolbarCustomization(FName InCustomization) {}

	/** Sets the visibility of the toolbar to switch between customizations */
	virtual void SetToolbarVisibility(bool bInVisible) {}

	/** Update the context of the widget panel */
	virtual void SetContext(const FOperatorStackEditorContext& InNewContext) {}

	/** Sets the keyframe handler to keyframe items properties */
	virtual void SetKeyframeHandler(TSharedPtr<IDetailKeyframeHandler> InKeyframeHandler) {}

	/** Sets the detail column sizes to have same column sizes across items */
	virtual void SetDetailColumnSize(TSharedPtr<FDetailColumnSizeData> InDetailColumnSize) {}

	/** Sets the panel tag to identify this panel */
	virtual void SetPanelTag(FName InTag) {}

	/** Forces a refresh of the current context if any */
	virtual void RefreshContext() {}

	/** Get the current context of this operator stack widget */
	virtual TSharedPtr<FOperatorStackEditorContext> GetContext() const
	{
		return nullptr;
	}

	/** Get the panel identifier, unique per widget */
	virtual int32 GetPanelId() const
	{
		return INDEX_NONE;
	}

	/** Get the panel tag */
	virtual FName GetPanelTag() const
	{
		return NAME_None;
	}
};